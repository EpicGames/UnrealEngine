// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "StructUtils/InstancedStructContainer.h"
#include "StructUtilsTestTypes.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#define LOCTEXT_NAMESPACE "StructUtilsTests"

namespace FInstancedStructContainerTest
{

struct FTest_BasicInstancedStructContainer : FAITestBase
{
	virtual bool InstantTest() override
	{
		FInstancedStructContainer Array;

		TArray<FConstStructView> TestStructs;
		TestStructs.Add(TBaseStructure<FTestStructSimple>::Get());
		TestStructs.Add(TBaseStructure<FVector>::Get());
		TestStructs.Add(TBaseStructure<FTestStructComplex>::Get());
		
		Array.Append(TestStructs);

		Array[1].Get<FVector>().X = 42.0;

		AITEST_EQUAL(TEXT("Should have 3 items"), Array.Num(), 3);
		AITEST_TRUE(TEXT("Item 0 should be FTestStructSimple"), Array[0].GetScriptStruct() == TBaseStructure<FTestStructSimple>::Get());
		AITEST_TRUE(TEXT("Item 1 should be FVector"), Array[1].GetScriptStruct() == TBaseStructure<FVector>::Get());
		AITEST_TRUE(TEXT("Item 2 should be FTestStructComplex"), Array[2].GetScriptStruct() == TBaseStructure<FTestStructComplex>::Get());
		AITEST_TRUE(TEXT("Item 1 should have X == 42.0"), FMath::IsNearlyEqual(Array[1].Get<FVector>().X, 42.0));

		Array[2].Get<FTestStructComplex>().StringArray.Add(TEXT("Foo"));
		
		TArray<FInstancedStruct> TestInstanced;
		TestInstanced.AddDefaulted_GetRef().InitializeAs<FTransform>();
		TestInstanced.AddDefaulted_GetRef().InitializeAs<FVector>(1,2,3);
		Array.Append(TestInstanced);

		AITEST_EQUAL(TEXT("Should have 5 items"), Array.Num(), 5);
		AITEST_TRUE(TEXT("Item 4 should have Z == 3.0"), FMath::IsNearlyEqual(Array[4].Get<FVector>().Z, 3.0));
		AITEST_TRUE(TEXT("Item 2 should have text Foo"), Array[2].Get<FTestStructComplex>().StringArray[0] == TEXT("Foo"));

		Array.RemoveAt(2, 1);
		AITEST_EQUAL(TEXT("Should have 4 items"), Array.Num(), 4);
		AITEST_TRUE(TEXT("Item 2 should be FTransform"), Array[2].GetScriptStruct() == TBaseStructure<FTransform>::Get());
		AITEST_TRUE(TEXT("Item 3 should be FVector"), Array[3].GetScriptStruct() == TBaseStructure<FVector>::Get());
		AITEST_TRUE(TEXT("Item 3 should have Z == 3.0"), FMath::IsNearlyEqual(Array[3].Get<FVector>().Z, 3.0));

		Array.SetNum(2);
		AITEST_EQUAL(TEXT("Should have 2 items"), Array.Num(), 2);

		FInstancedStructContainer Array2;
		Array2.Append({ TBaseStructure<FQuat>::Get(), TBaseStructure<FTestStructSimple1>::Get() });
		Array.InsertAt(1, Array2);
		AITEST_EQUAL(TEXT("Should have 4 items"), Array.Num(), 4);
		AITEST_TRUE(TEXT("Item 0 should be FTestStructSimple"), Array[0].GetScriptStruct() == TBaseStructure<FTestStructSimple>::Get());
		AITEST_TRUE(TEXT("Item 1 should be FQuat"), Array[1].GetScriptStruct() == TBaseStructure<FQuat>::Get());
		AITEST_TRUE(TEXT("Item 2 should be FTestStructSimple1"), Array[2].GetScriptStruct() == TBaseStructure<FTestStructSimple1>::Get());
		AITEST_TRUE(TEXT("Item 3 should be FVector"), Array[3].GetScriptStruct() == TBaseStructure<FVector>::Get());
		AITEST_TRUE(TEXT("Item 3 should have X == 42.0"), FMath::IsNearlyEqual(Array[3].Get<FVector>().X, 42.0));

		Array.RemoveAt(0, 3);
		AITEST_EQUAL(TEXT("Should have 1 item"), Array.Num(), 1);
		AITEST_TRUE(TEXT("Item 0 should be FVector"), Array[0].GetScriptStruct() == TBaseStructure<FVector>::Get());
		AITEST_TRUE(TEXT("Item 0 should have X == 42.0"), FMath::IsNearlyEqual(Array[0].Get<FVector>().X, 42.0));

		Array.InsertAt(0, Array2);
		AITEST_EQUAL(TEXT("Should have 1 item"), Array.Num(), 3);
		AITEST_TRUE(TEXT("Item 2 should have X == 42.0"), FMath::IsNearlyEqual(Array[2].Get<FVector>().X, 42.0));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_BasicInstancedStructContainer, "System.StructUtils.InstancedStructContainer.Basic");

struct FTest_SerializeInstancedStructContainer : FAITestBase
{
	virtual bool InstantTest() override
	{
		FInstancedStructContainer Array;

		TArray<FInstancedStruct> TestInstanced;
		TestInstanced.AddDefaulted_GetRef().InitializeAs<FTransform>();
		TestInstanced.AddDefaulted_GetRef().InitializeAs<FVector>(1,2,3);
		TestInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructComplex>();
		Array.Append(TestInstanced);

		Array[2].Get<FTestStructComplex>().StringArray.Add(TEXT("Foo"));

		AITEST_EQUAL(TEXT("Should have 3 items"), Array.Num(), 3);
		AITEST_TRUE(TEXT("Item 2 should have text Foo"), Array[2].Get<FTestStructComplex>().StringArray[0] == TEXT("Foo"));

		TArray<uint8> Memory;
		
		FMemoryWriter Writer(Memory);
		FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
		const bool bSaveResult = Array.Serialize(WriterProxy);
		AITEST_TRUE(TEXT("Saving should succeed"), bSaveResult);

		FMemoryReader Reader(Memory);
		FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
		const bool bLoadResult = Array.Serialize(ReaderProxy);
		AITEST_TRUE(TEXT("Loading to same array should succeed"), bLoadResult);

		FInstancedStructContainer Array2;
		FMemoryReader Reader2(Memory);
		FObjectAndNameAsStringProxyArchive ReaderProxy2(Reader2, /*bInLoadIfFindFails*/true);
		const bool bLoadResult2 = Array2.Serialize(ReaderProxy2);
		AITEST_TRUE(TEXT("Loading to Array2 should succeed"), bLoadResult2);
		AITEST_EQUAL(TEXT("Array2 should have 3 items"), Array2.Num(), 3);
		AITEST_TRUE(TEXT("Array2 item 2 should have text Foo"), Array2[2].Get<FTestStructComplex>().StringArray[0] == TEXT("Foo"));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_SerializeInstancedStructContainer, "System.StructUtils.InstancedStructContainer.Serialize");

struct FTest_RangedForInstancedStructContainer : FAITestBase
{
	virtual bool InstantTest() override
	{
		FInstancedStructContainer Array;
		const FInstancedStructContainer& ConstArray = Array;

		TArray<FInstancedStruct> TestInstanced;
		TestInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructComplex>(TEXT("0"));
		TestInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructComplex>(TEXT("1"));
		TestInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructComplex>(TEXT("2"));
		Array.Append(TestInstanced);

		int32 Count = 0;
		for (FStructView Item : Array)
		{
			if (Item.GetPtr<FTestStructComplex>())
			{
				Count++;
			}
		}
		AITEST_EQUAL(TEXT("Count should be 3"), Count, 3);

		for (FConstStructView Item : ConstArray)
		{
			if (Item.GetPtr<const FTestStructComplex>())
			{
				Count++;
			}
		}
		AITEST_EQUAL(TEXT("Count should be 6"), Count, 6);

		int32 ItemIndex = 0;
		for (FConstStructView Item : ConstArray)
		{
			AITEST_EQUAL("Should iterate in order", Item, ConstArray[ItemIndex++]);
		}

		FInstancedStructContainer::FIterator Iter = Array.CreateIterator();
		++Iter;
		Iter.RemoveCurrent();
		AITEST_EQUAL(TEXT("Array should have 2 items"), Array.Num(), 2);
		AITEST_EQUAL(TEXT("Iter should be at 0"), Iter.GetIndex(), 0);
		AITEST_TRUE(TEXT("Iter should be true"), (bool)Iter);
		AITEST_EQUAL(TEXT("Array[0] should be 0"), Array[0].Get<FTestStructComplex>().String, TEXT("0"));
		AITEST_EQUAL(TEXT("Array[1] should be 2"), Array[1].Get<FTestStructComplex>().String, TEXT("2"));

		++Iter;
		++Iter;
		AITEST_FALSE(TEXT("Iter should be false"), (bool)Iter);

		for (FInstancedStructContainer::FIterator Iter2 = Array.CreateIterator(); Iter2; ++Iter2)
		{
			Iter2.RemoveCurrent();
		}
		AITEST_EQUAL(TEXT("Array should have 0 items"), Array.Num(), 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_RangedForInstancedStructContainer, "System.StructUtils.InstancedStructContainer.RangedFor");

// Test that FInstancedStructContainer serialization correctly preserves values both with explicit
// Defaults and with nullptr DefaultsStruct / nullptr Defaults (C++ struct defaults fallback).
// The same four-item layout is exercised for each path:
//   Item 0: FTestStructSimpleNonZeroDefault, Float = 100.0f (== default 100.0f) — not written.
//   Item 1: FTestStructSimpleNonZeroDefault, Float =   0.0f (!= default 100.0f) — written.
//   Item 2: FTestStructSimple,               Float =   0.0f (== default   0.0f) — not written.
//   Item 3: FTestStructSimple,               Float =  50.0f (!= default   0.0f) — written.
struct FTest_SerializeInstancedStructContainerWithDefaults : FAITestBase
{
	virtual bool InstantTest() override
	{
		// Helper lambda that builds a container with the four test items.
		auto MakeContainer = [](FInstancedStructContainer& Container)
		{
			TArray<FInstancedStruct> TestInstanced;
			TestInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructSimpleNonZeroDefault>();                                          // Item 0: Float=100.0f
			FTestStructSimpleNonZeroDefault& Item1 = TestInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructSimpleNonZeroDefault>(); // Item 1
			Item1.Float = 0.0f;
			TestInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructSimple>();                                                        // Item 2: Float=0.0f
			FTestStructSimple& Item3 = TestInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructSimple>();                             // Item 3
			Item3.Float = 50.0f;
			Container.Append(TestInstanced);
		};

		// Helper lambda that scrambles all four items so a subsequent load must actively restore them.
		auto ScrambleItems = [](FInstancedStructContainer& Container)
		{
			Container[0].Get<FTestStructSimpleNonZeroDefault>().Float = 0.0f;   // was 100.0f
			Container[1].Get<FTestStructSimpleNonZeroDefault>().Float = 100.0f; // was 0.0f
			Container[2].Get<FTestStructSimple>().Float = 50.0f;                // was 0.0f
			Container[3].Get<FTestStructSimple>().Float = 0.0f;                 // was 50.0f
		};

		// --- Pass 1: explicit DefaultContainer ---
		{
			// Build the defaults container — one entry per item, representing archetype/CDO values.
			FInstancedStructContainer DefaultContainer;
			{
				TArray<FInstancedStruct> DefaultInstanced;
				DefaultInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructSimpleNonZeroDefault>(); // Float=100.0f
				DefaultInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructSimpleNonZeroDefault>(); // Float=100.0f
				DefaultInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructSimple>();               // Float=0.0f
				DefaultInstanced.AddDefaulted_GetRef().InitializeAs<FTestStructSimple>();               // Float=0.0f
				DefaultContainer.Append(DefaultInstanced);
			}

			FInstancedStructContainer Container;
			MakeContainer(Container);
			AITEST_EQUAL(TEXT("Explicit: Should have 4 items before save"), Container.Num(), 4);

			TArray<uint8> Memory;
			FMemoryWriter Writer(Memory);
			FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
			AITEST_TRUE(TEXT("Explicit: saving should succeed"), Container.Serialize(WriterProxy, FInstancedStructContainer::StaticStruct(), &DefaultContainer));

			ScrambleItems(Container);

			FMemoryReader Reader(Memory);
			FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
			AITEST_TRUE(TEXT("Explicit: loading should succeed"), Container.Serialize(ReaderProxy, FInstancedStructContainer::StaticStruct(), &DefaultContainer));

			AITEST_EQUAL(TEXT("Explicit: Should have 4 items after load"), Container.Num(), 4);
			AITEST_EQUAL(TEXT("Explicit: Item 0 Float should be restored to default 100.0f"), 100.0f, Container[0].Get<FTestStructSimpleNonZeroDefault>().Float);
			AITEST_EQUAL(TEXT("Explicit: Item 1 Float should be restored to saved zero 0.0f"),   0.0f, Container[1].Get<FTestStructSimpleNonZeroDefault>().Float);
			AITEST_EQUAL(TEXT("Explicit: Item 2 Float should be restored to default 0.0f"),      0.0f, Container[2].Get<FTestStructSimple>().Float);
			AITEST_EQUAL(TEXT("Explicit: Item 3 Float should be restored to saved value 50.0f"), 50.0f, Container[3].Get<FTestStructSimple>().Float);
		}

		// --- Pass 2: nullptr DefaultsStruct and nullptr Defaults ---
		// The serializer falls back to a temporary C++ default instance per item, so results
		// are identical to Pass 1 because the explicit defaults above equal the C++ defaults.
		{
			FInstancedStructContainer Container;
			MakeContainer(Container);
			AITEST_EQUAL(TEXT("Nullptr: Should have 4 items before save"), Container.Num(), 4);

			TArray<uint8> Memory;
			FMemoryWriter Writer(Memory);
			FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
			AITEST_TRUE(TEXT("Nullptr: saving should succeed"), Container.Serialize(WriterProxy, nullptr, nullptr));

			ScrambleItems(Container);

			FMemoryReader Reader(Memory);
			FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
			AITEST_TRUE(TEXT("Nullptr: loading should succeed"), Container.Serialize(ReaderProxy, nullptr, nullptr));

			AITEST_EQUAL(TEXT("Nullptr: Should have 4 items after load"), Container.Num(), 4);
			AITEST_EQUAL(TEXT("Nullptr: Item 0 Float should be restored to C++ default 100.0f"), 100.0f, Container[0].Get<FTestStructSimpleNonZeroDefault>().Float);
			AITEST_EQUAL(TEXT("Nullptr: Item 1 Float should be restored to saved zero 0.0f"),      0.0f, Container[1].Get<FTestStructSimpleNonZeroDefault>().Float);
			AITEST_EQUAL(TEXT("Nullptr: Item 2 Float should be restored to C++ default 0.0f"),     0.0f, Container[2].Get<FTestStructSimple>().Float);
			AITEST_EQUAL(TEXT("Nullptr: Item 3 Float should be restored to saved value 50.0f"),   50.0f, Container[3].Get<FTestStructSimple>().Float);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_SerializeInstancedStructContainerWithDefaults, "System.StructUtils.InstancedStructContainer.SerializeWithDefaults");

} // FInstancedStructContainerTest

#undef LOCTEXT_NAMESPACE
