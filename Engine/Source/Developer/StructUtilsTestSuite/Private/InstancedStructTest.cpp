// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "StructUtils/SharedStruct.h"
#include "StructUtilsTestTypes.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#define LOCTEXT_NAMESPACE "StructUtilsTests"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FInstancedStructTest
{
	struct FTest_InstancedStructCreate : FAITestBase
	{
		virtual bool InstantTest() override
		{
			constexpr float Val = 99.f;
			
			{
				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimpleNonZeroDefault>();

				AITEST_EQUAL("FInstancedStruct default initialized from Make should have same value as default constructed", FTestStructSimpleNonZeroDefault(), InstancedStruct.Get<FTestStructSimpleNonZeroDefault>());
			}

			{
				FTestStructSimple Simple(Val);

				FInstancedStruct InstancedStruct = FInstancedStruct::Make(Simple);

				AITEST_EQUAL("FInstancedStruct initialized from Make should have value of FTestStructSimple its initiliazed from", Val, InstancedStruct.Get<FTestStructSimple>().Float);
			}

			{
				FTestStructSimple Simple(Val);
				FStructView StructView = FStructView::Make(Simple);
				FInstancedStruct InstancedStruct(StructView);

				AITEST_EQUAL("FInstancedStruct initialized from Make should have value of FStructView its initiliazed from", Val, InstancedStruct.Get<FTestStructSimple>().Float);
			}
			
			{
				FTestStructSimple Simple(Val);
				TConstStructView<FTestStructSimple> ConstStructView(Simple);
				TInstancedStruct<FTestStructSimple> InstancedStruct(ConstStructView);

				AITEST_EQUAL("TInstancedStruct initialized from Make should have value of TConstStructView its initiliazed from", Val, InstancedStruct.Get<FTestStructSimple>().Float);
			}

			{
				FTestStructSimple Simple(Val);

				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimple>(Val);

				AITEST_EQUAL("FInstancedStruct initialized from Make should have value reflecting TArgs", Val, InstancedStruct.Get<FTestStructSimple>().Float);
			}

			{
				FTestStructSimple Simple(Val);

				FInstancedStruct InstancedStruct;
				InstancedStruct.InitializeAs<FTestStructSimple>(Val);

				AITEST_EQUAL("FInstancedStruct initialized from Make should have value reflecting TArgs", Val, InstancedStruct.Get<FTestStructSimple>().Float);
			}

			{
				FTestStructSimple Simple(Val);

				FInstancedStruct InstancedStruct;
				InstancedStruct.InitializeAs<FTestStructSimple>(Val);
				AITEST_EQUAL("FInstancedStruct initialized from InitializeAs should have value reflecting TArgs", Val, InstancedStruct.Get<FTestStructSimple>().Float);

				InstancedStruct.InitializeAs<FTestStructSimpleNonZeroDefault>();
				AITEST_EQUAL("FInstancedStruct initialized from InitializeAs should have same value as default constructed", FTestStructSimpleNonZeroDefault(), InstancedStruct.Get<FTestStructSimpleNonZeroDefault>());

				InstancedStruct.InitializeAs(nullptr);
				AITEST_FALSE("FInstancedStruct initialized from InitializeAs with empty struct should not be valid", InstancedStruct.IsValid());
			}

			return true;
		}
	};

	IMPLEMENT_AI_INSTANT_TEST(FTest_InstancedStructCreate, "System.StructUtils.InstancedStruct.Make");

	struct FTest_InstancedStructBasic : FAITestBase
	{
		virtual bool InstantTest() override
		{
			{
				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimple>();
				FInstancedStruct InstancedStruct2(InstancedStruct);

				AITEST_EQUAL("InstancedStruct and InstancedStruct2 should be equal from copy construction", InstancedStruct, InstancedStruct2);
			}

			{
				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimple>();
				FInstancedStruct InstancedStruct2;

				InstancedStruct2 = InstancedStruct;

				AITEST_EQUAL("FInstancedStruct and FInstancedStruct should be equal from copy assignment", InstancedStruct, InstancedStruct2);
			}

			{
				FInstancedStruct InstancedStruct;
				AITEST_FALSE("Default constructed FInstancedStruct should IsValid() == false", InstancedStruct.IsValid());
			}

			{
				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimple>();
				AITEST_TRUE("FInstancedStruct created to a specific struct type should be IsValid()", InstancedStruct.IsValid());
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTest_InstancedStructBasic, "System.StructUtils.InstancedStruct.Basic");

	struct FTest_InstancedStructCustomScriptStruct : FAITestBase
	{
		virtual bool InstantTest() override
		{
			// Create the TestObject before UScriptStruct, so that CustomStruct gets destroyed first.
			TWeakObjectPtr<UTestObjectWithInstanceStruct> TestObject = NewObject<UTestObjectWithInstanceStruct>();
			check(TestObject.IsValid());

			TWeakObjectPtr<UScriptStruct> CustomStruct = NewObject<UScriptStruct>();
			check(CustomStruct.IsValid());

			FIntProperty* IntProp = new FIntProperty(CustomStruct.Get(), FName("Int"));
			check(IntProp);
			CustomStruct->AddCppProperty(IntProp);

			FStrProperty* StrProp = new FStrProperty(CustomStruct.Get(), FName("String"));
			check(StrProp);
			CustomStruct->AddCppProperty(StrProp);

			CustomStruct->SetSuperStruct(nullptr);
			CustomStruct->Bind();
			CustomStruct->StaticLink(/*RelinkExistingProperties*/true);

			
			TestObject->Value.InitializeAs(CustomStruct.Get());
			AITEST_TRUE("FInstancedStruct created to a specific struct type should be IsValid()", TestObject->Value.IsValid());

			// CustomStruct and TestObject should both get collected.
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

			AITEST_FALSE("CustomStruct should not be valid", CustomStruct.IsValid());
			AITEST_FALSE("TestObject should not be valid", TestObject.IsValid());
			
			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTest_InstancedStructCustomScriptStruct, "System.StructUtils.InstancedStruct.CustomScriptStruct");

	// Test that serialization with explicit Defaults correctly preserves values across four cases,
	// and that passing nullptr/nullptr falls back to C++ defaults with the same correctness.
	// The Defaults FInstancedStruct drives both what gets skipped during save (delta) and what
	// gets used as the base values during load (InitializeAs is called with the default memory
	// before the stored delta is applied).
	//   Cases 1-4: explicit Defaults.
	//   Cases 5-8: nullptr DefaultsStruct / nullptr Defaults (falls back to C++ struct defaults).
	//   Case 1/5: FTestStructSimpleNonZeroDefault, Float == default (100.0f) — not written.
	//   Case 2/6: FTestStructSimpleNonZeroDefault, Float != default (0.0f vs 100.0f) — written.
	//   Case 3/7: FTestStructSimple, Float == default (0.0f) — not written.
	//   Case 4/8: FTestStructSimple, Float != default (50.0f vs 0.0f) — written.
	struct FTest_InstancedStructSerializeWithDefaults : FAITestBase
	{
		virtual bool InstantTest() override
		{
			// Case 1: FTestStructSimpleNonZeroDefault, Float at the provided default (100.0f).
			// Not written; load must restore it from the default.
			{
				const FInstancedStruct DefaultStruct = FInstancedStruct::Make<FTestStructSimpleNonZeroDefault>(); // Float=100.0f

				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimpleNonZeroDefault>(); // Float=100.0f

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case1: saving should succeed"), InstancedStruct.Serialize(WriterProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				InstancedStruct.GetMutable<FTestStructSimpleNonZeroDefault>().Float = 0.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case1: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				AITEST_EQUAL(TEXT("Case1: Float should be restored to default 100.0f"), 100.0f, InstancedStruct.Get<FTestStructSimpleNonZeroDefault>().Float);
			}

			// Case 2: FTestStructSimpleNonZeroDefault, Float differs from the provided default (0.0f vs 100.0f).
			// Written; load must restore the saved zero value.
			{
				const FInstancedStruct DefaultStruct = FInstancedStruct::Make<FTestStructSimpleNonZeroDefault>(); // Float=100.0f

				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimpleNonZeroDefault>();
				InstancedStruct.GetMutable<FTestStructSimpleNonZeroDefault>().Float = 0.0f;

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case2: saving should succeed"), InstancedStruct.Serialize(WriterProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				InstancedStruct.GetMutable<FTestStructSimpleNonZeroDefault>().Float = 100.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case2: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				AITEST_EQUAL(TEXT("Case2: Float should be restored to saved zero 0.0f"), 0.0f, InstancedStruct.Get<FTestStructSimpleNonZeroDefault>().Float);
			}

			// Case 3: FTestStructSimple, Float at the provided default (0.0f).
			// Not written; load must restore it from the default.
			{
				const FInstancedStruct DefaultStruct = FInstancedStruct::Make<FTestStructSimple>(); // Float=0.0f

				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimple>(); // Float=0.0f

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case3: saving should succeed"), InstancedStruct.Serialize(WriterProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				InstancedStruct.GetMutable<FTestStructSimple>().Float = 50.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case3: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				AITEST_EQUAL(TEXT("Case3: Float should be restored to default 0.0f"), 0.0f, InstancedStruct.Get<FTestStructSimple>().Float);
			}

			// Case 4: FTestStructSimple, Float differs from the provided default (50.0f vs 0.0f).
			// Written; load must restore the saved non-zero value.
			{
				const FInstancedStruct DefaultStruct = FInstancedStruct::Make<FTestStructSimple>(); // Float=0.0f

				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimple>(50.0f);

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case4: saving should succeed"), InstancedStruct.Serialize(WriterProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				InstancedStruct.GetMutable<FTestStructSimple>().Float = 0.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case4: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				AITEST_EQUAL(TEXT("Case4: Float should be restored to saved value 50.0f"), 50.0f, InstancedStruct.Get<FTestStructSimple>().Float);
			}

			// Cases 5-8: same four scenarios with nullptr DefaultsStruct and nullptr Defaults.
			// The serializer falls back to a temporary C++ default instance, so the behaviour is
			// identical to the explicit-defaults cases above (whose defaults match the C++ defaults).

			// Case 5: FTestStructSimpleNonZeroDefault, Float at C++ default (100.0f). Not written.
			{
				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimpleNonZeroDefault>(); // Float=100.0f

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case5: saving should succeed"), InstancedStruct.Serialize(WriterProxy, nullptr, nullptr));

				InstancedStruct.GetMutable<FTestStructSimpleNonZeroDefault>().Float = 0.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case5: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, nullptr, nullptr));

				AITEST_EQUAL(TEXT("Case5: Float should be restored to C++ default 100.0f"), 100.0f, InstancedStruct.Get<FTestStructSimpleNonZeroDefault>().Float);
			}

			// Case 6: FTestStructSimpleNonZeroDefault, Float differs from C++ default (0.0f vs 100.0f). Written.
			{
				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimpleNonZeroDefault>();
				InstancedStruct.GetMutable<FTestStructSimpleNonZeroDefault>().Float = 0.0f;

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case6: saving should succeed"), InstancedStruct.Serialize(WriterProxy, nullptr, nullptr));

				InstancedStruct.GetMutable<FTestStructSimpleNonZeroDefault>().Float = 100.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case6: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, nullptr, nullptr));

				AITEST_EQUAL(TEXT("Case6: Float should be restored to saved zero 0.0f"), 0.0f, InstancedStruct.Get<FTestStructSimpleNonZeroDefault>().Float);
			}

			// Case 7: FTestStructSimple, Float at C++ default (0.0f). Not written.
			{
				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimple>(); // Float=0.0f

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case7: saving should succeed"), InstancedStruct.Serialize(WriterProxy, nullptr, nullptr));

				InstancedStruct.GetMutable<FTestStructSimple>().Float = 50.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case7: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, nullptr, nullptr));

				AITEST_EQUAL(TEXT("Case7: Float should be restored to C++ default 0.0f"), 0.0f, InstancedStruct.Get<FTestStructSimple>().Float);
			}

			// Case 8: FTestStructSimple, Float differs from C++ default (50.0f vs 0.0f). Written.
			{
				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimple>(50.0f);

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case8: saving should succeed"), InstancedStruct.Serialize(WriterProxy, nullptr, nullptr));

				InstancedStruct.GetMutable<FTestStructSimple>().Float = 0.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case8: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, nullptr, nullptr));

				AITEST_EQUAL(TEXT("Case8: Float should be restored to saved value 50.0f"), 50.0f, InstancedStruct.Get<FTestStructSimple>().Float);
			}

			// Case 9: Polymorphic defaults (regression test for UE-384159). The Defaults hold the *base*
			// type while the value is a *subclass* left at its constructor default (Float == 100.0f), so
			// delta serialization omits Float on save. Because the serialized type differs from the
			// Defaults' type, load must reconstruct from the subclass's own constructor default. The
			// previous load path treated "subclass of the default's type" as compatible and copied the
			// larger subclass layout out of the smaller base-typed default memory - an out-of-bounds read
			// that left the omitted Float as garbage. Mirrors the bSameStruct exact-match used on save.
			{
				const FInstancedStruct DefaultStruct = FInstancedStruct::Make<FTestStructSimpleBase>();

				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimpleNonZeroDefault>(); // Float=100.0f

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case9: saving should succeed"), InstancedStruct.Serialize(WriterProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				// Stomp the in-memory value so a correct load has to actively restore the default.
				InstancedStruct.GetMutable<FTestStructSimpleNonZeroDefault>().Float = 0.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case9: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				AITEST_EQUAL(TEXT("Case9: subclass Float should be restored to its own default 100.0f, not out-of-bounds garbage"), 100.0f, InstancedStruct.Get<FTestStructSimpleNonZeroDefault>().Float);
			}

			// Case 10: Same polymorphic-default setup, but the subclass value differs from its default
			// (Float == 50.0f), so it IS written. This path always worked (the written value overwrites
			// whatever load seeds), and must keep working after the fix.
			{
				const FInstancedStruct DefaultStruct = FInstancedStruct::Make<FTestStructSimpleBase>();

				FInstancedStruct InstancedStruct = FInstancedStruct::Make<FTestStructSimpleNonZeroDefault>();
				InstancedStruct.GetMutable<FTestStructSimpleNonZeroDefault>().Float = 50.0f;

				TArray<uint8> Memory;
				FMemoryWriter Writer(Memory);
				FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
				AITEST_TRUE(TEXT("Case10: saving should succeed"), InstancedStruct.Serialize(WriterProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				InstancedStruct.GetMutable<FTestStructSimpleNonZeroDefault>().Float = 0.0f;

				FMemoryReader Reader(Memory);
				FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
				AITEST_TRUE(TEXT("Case10: loading should succeed"), InstancedStruct.Serialize(ReaderProxy, FInstancedStruct::StaticStruct(), &DefaultStruct));

				AITEST_EQUAL(TEXT("Case10: subclass Float should be restored to saved value 50.0f"), 50.0f, InstancedStruct.Get<FTestStructSimpleNonZeroDefault>().Float);
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTest_InstancedStructSerializeWithDefaults, "System.StructUtils.InstancedStruct.SerializeWithDefaults");

}


UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
