// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "StructUtilsTestTypes.h"
#include "StructUtils/PropertyBagMapRef.h"

#define LOCTEXT_NAMESPACE "StructUtilsTests"

namespace FPropertyBagTest
{

struct FTest_CreatePropertyBag : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName IsHotName(TEXT("bIsHot"));
		static const FName TemperatureName(TEXT("Temperature"));
		static const FName CountName(TEXT("Count"));
		static const FName UInt32Name(TEXT("Unsigned32"));
		static const FName UInt64Name(TEXT("Unsigned64"));

		FInstancedPropertyBag Bag;

		Bag.AddProperty(IsHotName, EPropertyBagPropertyType::Bool);
		AITEST_TRUE(TEXT("Should have bIsHot property"), Bag.FindPropertyDescByName(IsHotName) != nullptr);
		AITEST_TRUE(TEXT("Set bIsHot should succeed"), Bag.SetValueBool(IsHotName, true) == EPropertyBagResult::Success);

		// Amend the bag with new properties.
		Bag.AddProperties({
			{ TemperatureName, EPropertyBagPropertyType::Float },
			{ CountName, EPropertyBagPropertyType::Int32 },
			{ UInt32Name, EPropertyBagPropertyType::UInt32 },
			{ UInt64Name, EPropertyBagPropertyType::UInt64 }
			});
		AITEST_TRUE(TEXT("Set Temperature should succeed"), Bag.SetValueFloat(TemperatureName, 451.0f) == EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("Set Count should succeed"), Bag.SetValueFloat(CountName, 42) == EPropertyBagResult::Success);

		AITEST_TRUE(TEXT("Set UInt32 should succeed"), Bag.SetValueUInt32(UInt32Name, MAX_uint32) == EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("Set UInt64 should succeed"), Bag.SetValueUInt64(UInt64Name, MAX_uint64) == EPropertyBagResult::Success);

		AITEST_TRUE(TEXT("UInt32 value could not be retrieved"), Bag.GetValueUInt32(UInt32Name).HasError() == false);
		AITEST_TRUE(TEXT("UInt32 value not correct"), Bag.GetValueUInt32(UInt32Name).GetValue() == MAX_uint32);

		AITEST_TRUE(TEXT("UInt64 value could not be retrieved"), Bag.GetValueUInt64(UInt64Name).HasError() == false);
		AITEST_TRUE(TEXT("UInt64 value not correct"), Bag.GetValueUInt64(UInt64Name).GetValue() == MAX_uint64);

		Bag.RemovePropertyByName(IsHotName);
		AITEST_TRUE(TEXT("Should not have bIsHot property"), Bag.FindPropertyDescByName(IsHotName) == nullptr);
		AITEST_TRUE(TEXT("Set bIsHot should not succeed"), Bag.SetValueBool(IsHotName, true) != EPropertyBagResult::Success);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_CreatePropertyBag, "System.StructUtils.PropertyBag.CreateBag");

struct FTest_MovePropertyBag : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName IsHotName(TEXT("bIsHot"));
		static const FName TemperatureName(TEXT("Temperature"));

		FInstancedPropertyBag Bag;

		Bag.AddProperty(IsHotName, EPropertyBagPropertyType::Bool);
		AITEST_TRUE(TEXT("Bag should have bIsHot property"), Bag.FindPropertyDescByName(IsHotName) != nullptr);
		AITEST_TRUE(TEXT("Bag set bIsHot should succeed"), Bag.SetValueBool(IsHotName, true) == EPropertyBagResult::Success);

		FInstancedPropertyBag Bag2(Bag);
		Bag2.AddProperty(TemperatureName, EPropertyBagPropertyType::Float);
		AITEST_TRUE(TEXT("Bag should have bIsHot property"), Bag.FindPropertyDescByName(IsHotName) != nullptr);
		AITEST_TRUE(TEXT("Bag2 should have bIsHot property"), Bag2.FindPropertyDescByName(IsHotName) != nullptr);
		AITEST_TRUE(TEXT("Bag2 should have Temperature property"), Bag2.FindPropertyDescByName(TemperatureName) != nullptr);

		FInstancedPropertyBag Bag3(MoveTemp(Bag));
		AITEST_TRUE(TEXT("Bag should not have bIsHot property"), Bag.FindPropertyDescByName(IsHotName) == nullptr);
		AITEST_TRUE(TEXT("Bag3 should have bIsHot property"), Bag3.FindPropertyDescByName(IsHotName) != nullptr);

		Bag = Bag2;
		AITEST_TRUE(TEXT("Bag should have bIsHot property"), Bag.FindPropertyDescByName(IsHotName) != nullptr);
		AITEST_TRUE(TEXT("Bag should have Temperature property"), Bag.FindPropertyDescByName(TemperatureName) != nullptr);
		AITEST_TRUE(TEXT("Bag2 should have bIsHot property"), Bag2.FindPropertyDescByName(IsHotName) != nullptr);
		AITEST_TRUE(TEXT("Bag2 should have Temperature property"), Bag2.FindPropertyDescByName(TemperatureName) != nullptr);

		Bag = MoveTemp(Bag2);
		AITEST_TRUE(TEXT("Bag should have bIsHot property"), Bag.FindPropertyDescByName(IsHotName) != nullptr);
		AITEST_TRUE(TEXT("Bag should have Temperature property"), Bag.FindPropertyDescByName(TemperatureName) != nullptr);
		AITEST_TRUE(TEXT("Bag2 should not have bIsHot property"), Bag2.FindPropertyDescByName(IsHotName) == nullptr);
		AITEST_TRUE(TEXT("Bag2 should not have Temperature property"), Bag2.FindPropertyDescByName(TemperatureName) == nullptr);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_MovePropertyBag, "System.StructUtils.PropertyBag.MoveBag");

struct FTest_MigrateProperty : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName TemperatureName(TEXT("Temperature"));

		FInstancedPropertyBag Bag;
		Bag.AddProperty(TemperatureName, EPropertyBagPropertyType::Float);
		AITEST_TRUE(TEXT("Bag should have Temperature property"), Bag.FindPropertyDescByName(TemperatureName) != nullptr);

		TValueOrError<float, EPropertyBagResult> FloatDefaultRes = Bag.GetValueFloat(TemperatureName);
		AITEST_TRUE(TEXT("Bag getting Temperature default value should succeed"), FloatDefaultRes.IsValid());
		AITEST_TRUE(TEXT("Bag Temperature default value should be 0"), FMath::IsNearlyEqual(FloatDefaultRes.GetValue(), 0.0f));
		
		AITEST_TRUE(TEXT("Bag set Temperature as float should succeed"), Bag.SetValueFloat(TemperatureName, 451.0f) == EPropertyBagResult::Success);
		TValueOrError<float, EPropertyBagResult> FloatRes = Bag.GetValueFloat(TemperatureName);
		AITEST_TRUE(TEXT("Bag Temperature as float should be 451"), FloatRes.IsValid() && FMath::IsNearlyEqual(FloatRes.GetValue(), 451.0f));

		AITEST_TRUE(TEXT("Bag set Temperature as int should succeed"), Bag.SetValueInt32(TemperatureName, 451) == EPropertyBagResult::Success);
		FloatRes = Bag.GetValueFloat(TemperatureName);
		AITEST_TRUE(TEXT("Bag Temperature as float should be 451"), FloatRes.IsValid() && FMath::IsNearlyEqual(FloatRes.GetValue(), 451.0f));
		TValueOrError<int64, EPropertyBagResult> Int64Res = Bag.GetValueInt64(TemperatureName);
		AITEST_TRUE(TEXT("Bag Temperature as int64 should be 451"), Int64Res.IsValid() && Int64Res.GetValue() == 451);

		Bag.AddProperty(TemperatureName, EPropertyBagPropertyType::Int32);
		const FPropertyBagPropertyDesc* TempDesc = Bag.FindPropertyDescByName(TemperatureName);
		AITEST_TRUE(TEXT("Temperature property should be int32"), TempDesc != nullptr && TempDesc->ValueType == EPropertyBagPropertyType::Int32);

		TValueOrError<int32, EPropertyBagResult> Int32Res = Bag.GetValueInt32(TemperatureName);
		AITEST_TRUE(TEXT("Bag Temperature as int32 should be 451"), Int32Res.IsValid() && Int32Res.GetValue() == 451);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_MigrateProperty, "System.StructUtils.PropertyBag.MigrateProperty");

struct FTest_Object : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName ObjectName(TEXT("Object"));

		UBagTestObject1* Test1 = NewObject<UBagTestObject1>();
		UBagTestObject2* Test2 = NewObject<UBagTestObject2>();
		UBagTestObject1Derived* Test1Derived = NewObject<UBagTestObject1Derived>();

		FInstancedPropertyBag Bag;
		Bag.AddProperty(ObjectName, EPropertyBagPropertyType::Object, UBagTestObject1::StaticClass());
		AITEST_TRUE(TEXT("Bag should have Object property"), Bag.FindPropertyDescByName(ObjectName) != nullptr);

		AITEST_TRUE(TEXT("Bag set Object to Test1Derived should succeed"), Bag.SetValueObject(ObjectName, Test1Derived) == EPropertyBagResult::Success);
		AITEST_FALSE(TEXT("Bag set Object to Test2 should fail"), Bag.SetValueObject(ObjectName, Test2) == EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("Bag set Object to Test1 should succeed"), Bag.SetValueObject(ObjectName, Test1) == EPropertyBagResult::Success);

		TValueOrError<UBagTestObject1*, EPropertyBagResult> Test1Res = Bag.GetValueObject<UBagTestObject1>(ObjectName);
		TValueOrError<UBagTestObject1Derived*, EPropertyBagResult> Test1DerivedRes = Bag.GetValueObject<UBagTestObject1Derived>(ObjectName);
		
		AITEST_TRUE(TEXT("Bag get Object as Test1 should succeed"), Test1Res.IsValid());
		AITEST_FALSE(TEXT("Bag get Object as Test1Derived should fail"), Test1DerivedRes.IsValid()); // Note: the current value is Test1, and Cast should fail.

		// Test conversion from Object to SoftObject
		Bag.AddProperty(ObjectName, EPropertyBagPropertyType::SoftObject, UBagTestObject1::StaticClass());
		TValueOrError<UBagTestObject1*, EPropertyBagResult> Test1Res2 = Bag.GetValueObject<UBagTestObject1>(ObjectName);
		AITEST_TRUE(TEXT("Bag get Object as Test1 should succeed after migration soft object"), Test1Res2.IsValid());
		AITEST_TRUE(TEXT("Bag get Object Test1 should be Test1 after migration soft object"), Test1Res2.GetValue() == Test1);

		// Test conversion from SoftObject to Object
		Bag.AddProperty(ObjectName, EPropertyBagPropertyType::Object, UBagTestObject1::StaticClass());
		TValueOrError<UBagTestObject1*, EPropertyBagResult> Test1Res3 = Bag.GetValueObject<UBagTestObject1>(ObjectName);
		AITEST_TRUE(TEXT("Bag get Object as Test1 should succeed after migration object"), Test1Res3.IsValid());
		AITEST_TRUE(TEXT("Bag get Object Test1 should be Test1 after migration object"), Test1Res3.GetValue() == Test1);

		// Test conversion from different type
		Bag.AddProperty(ObjectName, EPropertyBagPropertyType::Object, UBagTestObject2::StaticClass());
		TValueOrError<UBagTestObject1*, EPropertyBagResult> Test1Res4 = Bag.GetValueObject<UBagTestObject1>(ObjectName);
		TValueOrError<UBagTestObject2*, EPropertyBagResult> Test2Res = Bag.GetValueObject<UBagTestObject2>(ObjectName);
		AITEST_FALSE(TEXT("Bag get Object as Test1 should fail after migration to test2"), Test1Res4.IsValid());
		AITEST_TRUE(TEXT("Bag get Object as Test1 should succeed after migration to test2"), Test2Res.IsValid());
		AITEST_TRUE(TEXT("Bag get Object Test2 should be null after migration to test2"), Test2Res.GetValue() == nullptr);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_Object, "System.StructUtils.PropertyBag.Object");

struct FTest_Struct : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName StructName(TEXT("Struct"));

		FInstancedPropertyBag Bag;
		Bag.AddProperty(StructName, EPropertyBagPropertyType::Struct, FTestStructSimple::StaticStruct());
		AITEST_TRUE(TEXT("Bag should have Struct property"), Bag.FindPropertyDescByName(StructName) != nullptr);

		FTestStructSimple Value;
		Value.Float = 42.0f;

		FTestStructComplex Value2;

		AITEST_TRUE(TEXT("Bag set Struct as struct view should succeed"), Bag.SetValueStruct(StructName, FConstStructView::Make(Value)) == EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("Bag set Struct with template should succeed"), Bag.SetValueStruct(StructName, Value) == EPropertyBagResult::Success);
		AITEST_FALSE(TEXT("Bag set Struct to complex as struct view should succeed"), Bag.SetValueStruct(StructName, FConstStructView::Make(Value2)) == EPropertyBagResult::Success);
		AITEST_FALSE(TEXT("Bag set Struct to complex  with template should succeed"), Bag.SetValueStruct(StructName, Value2) == EPropertyBagResult::Success);
		
		TValueOrError<FStructView, EPropertyBagResult> Res1 = Bag.GetValueStruct(StructName);
		TValueOrError<FTestStructSimple*, EPropertyBagResult> Res2 = Bag.GetValueStruct<FTestStructSimple>(StructName);
		TValueOrError<FTestStructSimpleBase*, EPropertyBagResult> Res3 = Bag.GetValueStruct<FTestStructSimpleBase>(StructName);
		TValueOrError<FTestStructComplex*, EPropertyBagResult> Res4 = Bag.GetValueStruct<FTestStructComplex>(StructName);
		
		AITEST_TRUE(TEXT("Bag get Struct as struct view should succeed"), Res1.IsValid());
		AITEST_TRUE(TEXT("Bag get Struct as simple should succeed"), Res2.IsValid());
		AITEST_TRUE(TEXT("Bag result value should be 42"), FMath::IsNearlyEqual(Res2.GetValue()->Float, 42.0f));
		AITEST_TRUE(TEXT("Bag get Struct as simple base should succeed"), Res3.IsValid());
		AITEST_FALSE(TEXT("Bag get Struct as complex should succeed"), Res4.IsValid());

		Bag.AddProperty(StructName, EPropertyBagPropertyType::Bool);
		TValueOrError<FStructView, EPropertyBagResult> MigRes1 = Bag.GetValueStruct(StructName);
		TValueOrError<FTestStructSimple*, EPropertyBagResult> MigRes2 = Bag.GetValueStruct<FTestStructSimple>(StructName);
		
		AITEST_FALSE(TEXT("Bag get Struct as struct view should fail after migration"), MigRes1.IsValid());
		AITEST_FALSE(TEXT("Bag get Struct as simple should succeed after migration"), MigRes2.IsValid());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_Struct, "System.StructUtils.PropertyBag.Struct");

struct FTest_Class : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName ClassName(TEXT("Class"));

		FInstancedPropertyBag Bag;
		Bag.AddProperty(ClassName, EPropertyBagPropertyType::Class, UBagTestObject1::StaticClass());
		AITEST_TRUE(TEXT("Bag should have Class property"), Bag.FindPropertyDescByName(ClassName) != nullptr);

		AITEST_TRUE(TEXT("Bag set Class to UBagTestObject1 should succeed"), Bag.SetValueClass(ClassName, UBagTestObject1::StaticClass()) == EPropertyBagResult::Success);
		AITEST_FALSE(TEXT("Bag set Class to UBagTestObject2 should fail"), Bag.SetValueClass(ClassName, UBagTestObject2::StaticClass()) == EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("Bag set Class to UBagTestObject1Derived should succeed"), Bag.SetValueClass(ClassName, UBagTestObject1Derived::StaticClass()) == EPropertyBagResult::Success);

		TValueOrError<UClass*, EPropertyBagResult> Res1 = Bag.GetValueClass(ClassName);
		AITEST_TRUE(TEXT("Bag get Class should succeed"), Res1.IsValid());
		AITEST_TRUE(TEXT("Bag Class result should be UBagTestObject1Derived"), Res1.GetValue() == UBagTestObject1Derived::StaticClass());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_Class, "System.StructUtils.PropertyBag.Class");

struct FTest_SubclassOf : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName PropertyName = GET_MEMBER_NAME_CHECKED(FTestStructWithSubClassOf, ClassProperty);
		const UScriptStruct& Struct = *FTestStructWithSubClassOf::StaticStruct();
		const FClassProperty& Property = *CastFieldChecked<FClassProperty>(Struct.FindPropertyByName(PropertyName));

		FInstancedPropertyBag Bag;
		Bag.AddProperty(PropertyName, &Property);
		const FPropertyBagPropertyDesc* BagPropertyDesc = Bag.FindPropertyDescByName(PropertyName);
		AITEST_NOT_NULL(TEXT("Expecting to find inserted property"), BagPropertyDesc);

#if WITH_EDITOR // FPropertyBagPropertyDesc::MetaClass is in WITH_EDITORONLY_DATA
		AITEST_NOT_NULL(TEXT("Created bag property MetaClass pointer"), BagPropertyDesc->MetaClass.Get());
		AITEST_EQUAL(TEXT("Created bag property MetaClass type"), BagPropertyDesc->MetaClass->GetFName(), Property.MetaClass->GetFName());
#endif // WITH_EDITOR

		AITEST_NOT_NULL(TEXT("Created bag property ValueTypeObject pointer"), BagPropertyDesc->ValueTypeObject.Get());
		AITEST_EQUAL(TEXT("Created bag property ValueTypeObject type"), BagPropertyDesc->ValueTypeObject->GetFName(), Property.MetaClass->GetFName());

		const FString ResultSuccess = UEnum::GetValueAsString(EPropertyBagResult::Success);

		EPropertyBagResult SetResult = Bag.SetValueClass(PropertyName, UBagTestObject1::StaticClass());
		AITEST_EQUAL(TEXT("SetValueClass result for allowed class")
			, FStringView(UEnum::GetValueAsString(SetResult))
			, FStringView(ResultSuccess));

		SetResult = Bag.SetValueClass(PropertyName, UClass::StaticClass());
		AITEST_NOT_EQUAL(TEXT("SetValueClass result for non-allowed class")
			, FStringView(UEnum::GetValueAsString(SetResult))
			, FStringView(ResultSuccess));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_SubclassOf, "System.StructUtils.PropertyBag.SubclassOf");

struct FTest_SoftClassPtr : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName PropertyName = GET_MEMBER_NAME_CHECKED(FTestStructWithSubClassOf, SoftClassProperty);
		const UScriptStruct& Struct = *FTestStructWithSubClassOf::StaticStruct();
		const FSoftClassProperty& Property = *CastFieldChecked<FSoftClassProperty>(Struct.FindPropertyByName(PropertyName));

		FInstancedPropertyBag Bag;
		Bag.AddProperty(PropertyName, &Property);
		const FPropertyBagPropertyDesc* BagPropertyDesc = Bag.FindPropertyDescByName(PropertyName);
		AITEST_NOT_NULL(TEXT("Expecting to find inserted property"), BagPropertyDesc);

#if WITH_EDITOR // FPropertyBagPropertyDesc::MetaClass is in WITH_EDITORONLY_DATA
		AITEST_NOT_NULL(TEXT("Created bag property MetaClass pointer"), BagPropertyDesc->MetaClass.Get());
		AITEST_EQUAL(TEXT("Created bag property MetaClass type"), BagPropertyDesc->MetaClass->GetFName(), Property.MetaClass->GetFName());
#endif // WITH_EDITOR

		AITEST_NOT_NULL(TEXT("Created bag property ValueTypeObject pointer"), BagPropertyDesc->ValueTypeObject.Get());
		AITEST_EQUAL(TEXT("Created bag property ValueTypeObject type"), BagPropertyDesc->ValueTypeObject->GetFName(), Property.MetaClass->GetFName());

		const FString ResultSuccess = UEnum::GetValueAsString(EPropertyBagResult::Success);

		EPropertyBagResult SetResult = Bag.SetValueSoftPath(PropertyName, UBagTestObject1::StaticClass());
		AITEST_EQUAL(TEXT("SetValueSoftPath result for allowed class")
			, FStringView(UEnum::GetValueAsString(SetResult))
			, FStringView(ResultSuccess));

		SetResult = Bag.SetValueSoftPath(PropertyName, UClass::StaticClass());
		AITEST_NOT_EQUAL(TEXT("SetValueSoftPath result for non-allowed class")
			, FStringView(UEnum::GetValueAsString(SetResult))
			, FStringView(ResultSuccess));

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_SoftClassPtr, "System.StructUtils.PropertyBag.SoftClassPtr");

struct FTest_Enum : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName EnumName(TEXT("Enum"));

		FInstancedPropertyBag Bag;
		Bag.AddProperty(EnumName, EPropertyBagPropertyType::Enum, StaticEnum<EPropertyBagTest1>());
		AITEST_TRUE(TEXT("Bag should have Enum property"), Bag.FindPropertyDescByName(EnumName) != nullptr);

		AITEST_TRUE(TEXT("Bag set Enum to Foo should succeed"), Bag.SetValueEnum(EnumName, EPropertyBagTest1::Foo) == EPropertyBagResult::Success);
		AITEST_FALSE(TEXT("Bag set Enum to Bongo should fail"), Bag.SetValueEnum(EnumName, EPropertyBagTest2::Bongo) == EPropertyBagResult::Success);
		
		TValueOrError<EPropertyBagTest1, EPropertyBagResult> Res1 = Bag.GetValueEnum<EPropertyBagTest1>(EnumName);
		TValueOrError<EPropertyBagTest2, EPropertyBagResult> Res2 = Bag.GetValueEnum<EPropertyBagTest2>(EnumName);
		
		AITEST_TRUE(TEXT("Bag get Enum should succeed"), Res1.IsValid());
		AITEST_TRUE(TEXT("Bag Enum result should be Byte"), Res1.GetValue() == EPropertyBagTest1::Foo);
		AITEST_FALSE(TEXT("Bag get Enum with different type should fail"), Res2.IsValid());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_Enum, "System.StructUtils.PropertyBag.Enum");

struct FTest_ContainerTypes : FAITestBase
{
	virtual bool InstantTest() override
	{
		FPropertyBagContainerTypes Container = { EPropertyBagContainerType::None, EPropertyBagContainerType::None };
		AITEST_TRUE(TEXT("Invalid Num Containers after creation."), Container.Num() == 0);
		AITEST_TRUE(TEXT("Invalid First Container type after creation."), Container.GetFirstContainerType() == EPropertyBagContainerType::None);

		Container.Add(EPropertyBagContainerType::Array);
		AITEST_TRUE(TEXT("Invalid num containers"), Container.Num() == 1);
		AITEST_TRUE(TEXT("Invalid First Container type."), Container.GetFirstContainerType() == EPropertyBagContainerType::Array);

		Container.Add(EPropertyBagContainerType::Array);
		AITEST_TRUE(TEXT("Invalid num containers"), Container.Num() == 2);

		const EPropertyBagContainerType HeadContainerType1 = Container.PopHead();
		AITEST_TRUE(TEXT("Invalid num containers"), Container.Num() == 1);
		AITEST_TRUE(TEXT("Invalid extracted head container 1"), HeadContainerType1 == EPropertyBagContainerType::Array);
		AITEST_TRUE(TEXT("Invalid first container type after removing Head 1"), Container.GetFirstContainerType() == EPropertyBagContainerType::Array);

		const EPropertyBagContainerType HeadContainerType2 = Container.PopHead();
		AITEST_TRUE(TEXT("Invalid num containers"), Container.Num() == 0);
		AITEST_TRUE(TEXT("Invalid extracted head container 2"), HeadContainerType2 == EPropertyBagContainerType::Array);
		AITEST_TRUE(TEXT("Invalid first container type after removing Head 2"), Container.GetFirstContainerType() == EPropertyBagContainerType::None);

		Container.Add(EPropertyBagContainerType::None);
		AITEST_TRUE(TEXT("Adding None sould not change Num containers"), Container.Num() == 0);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_ContainerTypes, "System.StructUtils.PropertyBag.ContainerTypes");

struct FTest_NestedArray : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName NestedInt32ArrayPropName(TEXT("NestedInt32ArrayProp"));
		static const TArray<TArray<int32>> NestedInt32ArrayTestValue = { { 1, 2, 3} };

		FInstancedPropertyBag Bag;

		// Set properties
		{
			Bag.AddContainerProperty(NestedInt32ArrayPropName, { EPropertyBagContainerType::Array, EPropertyBagContainerType::Array }, EPropertyBagPropertyType::Int32, nullptr);

			AITEST_TRUE(TEXT("Missing Float Array property in the Bag."), Bag.FindPropertyDescByName(NestedInt32ArrayPropName) != nullptr);
		}

		// Check Default Value
		{
			TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> NestedInt32ArrayDefaultResult = Bag.GetArrayRef(NestedInt32ArrayPropName);

			AITEST_TRUE(TEXT("Bag getting Nested Int32 Array default value should succeed."), NestedInt32ArrayDefaultResult.IsValid());

			if (NestedInt32ArrayDefaultResult.IsValid())
			{
				AITEST_TRUE(TEXT("Bag Nested Int32 Array default value incorrect size"), NestedInt32ArrayDefaultResult.GetValue().Num() == 0);
			}
		}

		// Set Nested Array values using FPropertyBagArrayRef interface
		{
			TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> NestedInt32ArrayMutable = Bag.GetMutableArrayRef(NestedInt32ArrayPropName);

			AITEST_TRUE(TEXT("Getting PropertyBag Nested Int32 Array should succeed."), NestedInt32ArrayMutable.IsValid());

			FPropertyBagArrayRef& PropertyBagArrayRef = NestedInt32ArrayMutable.GetValue();

			const int32 NumArrays = NestedInt32ArrayTestValue.Num();
			PropertyBagArrayRef.AddValues(NumArrays);

			for (int32 n = 0; n < NumArrays; n++)
			{
				TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> InnerArrayTestResult = PropertyBagArrayRef.GetMutableNestedArrayRef(n);
					
				AITEST_TRUE(TEXT("Getting PropertyBag Nested Inner Int32 Array should succeed."), InnerArrayTestResult.IsValid());
					
				FPropertyBagArrayRef& InnerPropertyBagArrayRef = InnerArrayTestResult.GetValue();

				const int32 NumElem = NestedInt32ArrayTestValue[n].Num();
				InnerPropertyBagArrayRef.AddUninitializedValues(NumElem);

				for (int32 i = 0; i < NumElem; i++)
				{
					const int32& value = NestedInt32ArrayTestValue[n][i];

					AITEST_TRUE(TEXT("Setting value to Nested Inner Array property failed."), InnerPropertyBagArrayRef.SetValueInt32(i, value) == EPropertyBagResult::Success);
				}
			}
		}

		// Test Nested Array Values using FPropertyBagArrayRef interface
		{
			TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> NestedInt32ArrayTestResult = Bag.GetArrayRef(NestedInt32ArrayPropName);

			AITEST_TRUE(TEXT("Getting PropertyBag Nested Int32 Array should succeed."), NestedInt32ArrayTestResult.IsValid());

			if (NestedInt32ArrayTestResult.IsValid())
			{
				const FPropertyBagArrayRef& NestedPropertyBagArrayRef = NestedInt32ArrayTestResult.GetValue();

				const int32 NumArrays = NestedPropertyBagArrayRef.Num();
				const int32 NumTestArrays = NestedInt32ArrayTestValue.Num();

				AITEST_TRUE(TEXT("Bag [%s] Nested Int32 Array Num value mismatch."), NumArrays == NumTestArrays);

				for (int32 n = 0; n < NumArrays; ++n)
				{
					TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> InnerArrayTestResult = NestedPropertyBagArrayRef.GetNestedArrayRef();

					AITEST_TRUE(TEXT("Getting PropertyBag Nested Inner Int32 Array should succeed."), InnerArrayTestResult.IsValid());

					const FPropertyBagArrayRef& PropertyBagArrayRef = InnerArrayTestResult.GetValue();

					const int32 NumElem = InnerArrayTestResult.GetValue().Num();
					for (int32 i = 0; i < NumElem; i++)
					{
						const TValueOrError<int32, EPropertyBagResult> Int32Res = PropertyBagArrayRef.GetValueInt32(i);

						AITEST_TRUE(TEXT("Getting Nested Array Element should succeed."), Int32Res.IsValid());

						if (Int32Res.IsValid())
						{
							AITEST_TRUE(TEXT("Nested Arrauy test value mismatch."), Int32Res.GetValue() == NestedInt32ArrayTestValue[n][i]);
						}
					}
				}
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_NestedArray, "System.StructUtils.PropertyBag.NestedArray");

struct FTest_GC : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName EnumName(TEXT("Enum"));

		UTestObjectWithPropertyBag* Obj = NewObject<UTestObjectWithPropertyBag>();
		Obj->Bag.AddProperty(EnumName, EPropertyBagPropertyType::Enum, StaticEnum<EPropertyBagTest1>());

		const UPropertyBag* BagStruct = Obj->Bag.GetPropertyBagStruct();
		check(BagStruct);
		
		const FString BagStructName = BagStruct->GetName();
		const FString ObjName = Obj->GetName();

		// Obj is unreachable, it should be collected by the GC.
		Obj = nullptr;
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		// The used property bag struct should exists after the GC.
		const UPropertyBag* ExistingObj = FindObject<UPropertyBag>(GetTransientPackage(), *ObjName);
		UPropertyBag* ExistingBagStruct1 = FindObject<UPropertyBag>(GetTransientPackage(), *BagStructName);

		AITEST_NULL(TEXT("Obj should have been released"), ExistingObj);
		AITEST_NOT_NULL(TEXT("Bag struct should exists after Obj released"), ExistingBagStruct1);

		// The next GC should collect the bag struct
		ExistingBagStruct1->ClearFlags(RF_Standalone);
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

		const UPropertyBag* ExistingBagStruct2 = FindObject<UPropertyBag>(GetTransientPackage(), *BagStructName);
		AITEST_NULL(TEXT("Bag struct should not exists after second GC"), ExistingBagStruct2);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_GC, "System.StructUtils.PropertyBag.GC");

struct FTest_Arrays : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName FloatArrayName(TEXT("FloatArray"));

		FInstancedPropertyBag Bag;
		Bag.AddProperties({
			{ FloatArrayName, EPropertyBagContainerType::Array, EPropertyBagPropertyType::Float },
		});

		TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> FloatArrayRes = Bag.GetArrayRef(FloatArrayName);
		AITEST_TRUE(TEXT("Get float array should succeed"), FloatArrayRes.IsValid());

		FPropertyBagArrayRef FloatArray = FloatArrayRes.GetValue();
		const int32 FloatIndex = FloatArray.AddValue();
		AITEST_TRUE(TEXT("Float array should have 1 item"), FloatArray.Num() == 1);

		const TValueOrError<float, EPropertyBagResult> GetDefaultFloatRes = FloatArray.GetValueFloat(FloatIndex);
		AITEST_TRUE(TEXT("Get float should succeed immediately after add"), GetDefaultFloatRes.IsValid());
		AITEST_TRUE(TEXT("Default value for Float should be 0.0f"), FMath::IsNearlyEqual(GetDefaultFloatRes.GetValue(), 0.0f));

		const EPropertyBagResult SetFloatRes = FloatArray.SetValueFloat(FloatIndex, 123.0f);
		AITEST_TRUE(TEXT("Set float should succeed"), SetFloatRes == EPropertyBagResult::Success);

		const TValueOrError<float, EPropertyBagResult> GetFloatRes = FloatArray.GetValueFloat(FloatIndex);
		AITEST_TRUE(TEXT("Get float should succeed"), GetFloatRes.IsValid());
		AITEST_TRUE(TEXT("Float value should be 123.0f"), FMath::IsNearlyEqual(GetFloatRes.GetValue(), 123.0f));

		const TValueOrError<float, EPropertyBagResult> GetFloatOOBRes = FloatArray.GetValueFloat(42);
		AITEST_FALSE(TEXT("Get float out of bounds should not succeed"), GetFloatOOBRes.IsValid());
		AITEST_TRUE(TEXT("Error should be our of bounds"), GetFloatOOBRes.GetError() == EPropertyBagResult::OutOfBounds);

		const EPropertyBagResult SetFloatOOBRes = FloatArray.SetValueFloat(-1, 0.0);
		AITEST_TRUE(TEXT("Set float out of bounds should return out of bounds"), SetFloatOOBRes == EPropertyBagResult::OutOfBounds);
		
		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_Arrays, "System.StructUtils.PropertyBag.Arrays");

struct FTest_Sets : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName FloatSetName(TEXT("FloatSet"));
		static const FName EnumSetName(TEXT("EnumSet"));
		static const FName StructSetName(TEXT("StructSet"));
		static const FName ObjectSetName(TEXT("ObjectSet"));
		static const FName ClassSetName(TEXT("ClassSet"));

		FInstancedPropertyBag Bag;
		Bag.AddProperties({
			{ FloatSetName, EPropertyBagContainerType::Set, EPropertyBagPropertyType::Float },
			{ EnumSetName, EPropertyBagContainerType::Set, EPropertyBagPropertyType::Enum, StaticEnum<EPropertyBagTest1>() },
			{ StructSetName, EPropertyBagContainerType::Set, EPropertyBagPropertyType::Struct, FTestStructHashable1::StaticStruct()},
			{ ObjectSetName, EPropertyBagContainerType::Set, EPropertyBagPropertyType::Object, UBagTestObject1::StaticClass()},
			{ ClassSetName, EPropertyBagContainerType::Set, EPropertyBagPropertyType::Class, UBagTestObject1::StaticClass()}
		});

		//Test Numeric Type Set
		TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> FloatSetRes = Bag.GetSetRef(FloatSetName);
		AITEST_TRUE(TEXT("Bag should have float set"), FloatSetRes.IsValid());

		FPropertyBagSetRef FloatSet = FloatSetRes.GetValue();
		float FloatValue1 = 1.f;
		TValueOrError<bool, EPropertyBagResult> EmptySetContainsResult = FloatSet.Contains(FloatValue1);
		AITEST_TRUE(TEXT("Float set contain result should have value"), EmptySetContainsResult.HasValue());
		AITEST_EQUAL(TEXT("Float set contain result should be false"), EmptySetContainsResult.GetValue(), false);

		const EPropertyBagResult SetFloatRes = FloatSet.AddValueFloat(FloatValue1);
		AITEST_EQUAL(TEXT("Float set should have 1 item"), FloatSet.Num(), 1);
		AITEST_EQUAL(TEXT("Set float should succeed"), SetFloatRes, EPropertyBagResult::Success);

		TValueOrError<bool, EPropertyBagResult> FilledSetContainsResult = FloatSet.Contains(FloatValue1);
		AITEST_TRUE(TEXT("Float set contain result should have value"), FilledSetContainsResult.HasValue());
		AITEST_EQUAL(TEXT("Float set contain result should be true"), FilledSetContainsResult.GetValue(), true);

		const float FloatValue2 = 2.f;
		const EPropertyBagResult NewSetFloatRes = FloatSet.AddValueFloat(FloatValue2);
		AITEST_TRUE(TEXT("Setting a new float value should succeed"), NewSetFloatRes == EPropertyBagResult::Success);
		AITEST_EQUAL(TEXT("Float set should have 2 items"), FloatSet.Num(), 2);

		const float FloatValue3 = 3.f;
		TValueOrError<bool, EPropertyBagResult> ContainsUnknownResult = FloatSet.Contains(FloatValue3);
		AITEST_EQUAL(TEXT("Calling Contains with a value not stored in the set should return false"), ContainsUnknownResult.GetValue(), false);

		EPropertyBagResult RemoveUnknownResult = FloatSet.Remove(FloatValue3);
		AITEST_EQUAL(TEXT("Calling Remove with a value not stored in the set should return a property not found error"), RemoveUnknownResult, EPropertyBagResult::PropertyNotFound);
		AITEST_EQUAL(TEXT("Float set should still 2 items after failed removal"), FloatSet.Num(), 2);

		const EPropertyBagResult AddExistingFloatRes = FloatSet.AddValueFloat(FloatValue2);
		AITEST_EQUAL(TEXT("Setting an existing element to a value already present in the set should return a duplicated value error"), AddExistingFloatRes, EPropertyBagResult::DuplicatedValue);

		EPropertyBagResult RemoveKnownResult = FloatSet.Remove(FloatValue2);
		AITEST_EQUAL(TEXT("Removing an element in the set should result in success"), RemoveKnownResult, EPropertyBagResult::Success);
		AITEST_EQUAL(TEXT("Float set should have 1 item after successfull removal"), FloatSet.Num(), 1);

		const int32 IntValue = 3;
		const EPropertyBagResult SetIntRes = FloatSet.AddValueInt32(IntValue);
		AITEST_EQUAL(TEXT("Setting a signed integer on a float set should succeed"), SetIntRes, EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("Float set should contain the new int value"), FloatSet.Contains(IntValue).GetValue());

		const uint32 UIntValue = 4;
		const EPropertyBagResult SetUIntRes = FloatSet.AddValueUInt32(UIntValue);
		AITEST_EQUAL(TEXT("Setting an unsigned integer on a float set should succeed"), SetUIntRes, EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("Float set should contain the new uint value"), FloatSet.Contains(UIntValue).GetValue());

		const FString TestString = TEXT("TestString");
		const EPropertyBagResult SetStringResult = FloatSet.AddValueString(TestString);
		AITEST_NOT_EQUAL(TEXT("Setting a string on a float set should not succeed"), SetStringResult, EPropertyBagResult::Success);
		AITEST_FALSE(TEXT("Float set should not contain the new string value"), FloatSet.Contains(TestString).GetValue());

		//Test Enum Set
		TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> EnumSetRes = Bag.GetSetRef(EnumSetName);
		AITEST_TRUE(TEXT("Bag should have Enum set property"), EnumSetRes.IsValid());
		FPropertyBagSetRef EnumSet = EnumSetRes.GetValue();

		AITEST_EQUAL(TEXT("Adding enum value to set should succeed"), EnumSet.AddValueEnum(EPropertyBagTest1::Foo), EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("We should be able to find the enum value we just added"), EnumSet.Contains(EPropertyBagTest1::Foo).GetValue());
		AITEST_EQUAL(TEXT("Adding a different enum value to set should succeed"), EnumSet.AddValueEnum(EPropertyBagTest1::Bar), EPropertyBagResult::Success);
		AITEST_EQUAL(TEXT("Adding an already store enum value should return a duplicated value error"), EnumSet.AddValueEnum(EPropertyBagTest1::Bar), EPropertyBagResult::DuplicatedValue);
		AITEST_EQUAL(TEXT("Adding value from a different enum type to set should return a type mismatch error"), EnumSet.AddValueEnum(EPropertyBagTest2::Bongo), EPropertyBagResult::TypeMismatch);
		AITEST_EQUAL(TEXT("Adding a non enum value to an enum set should return a type mismatch error"), EnumSet.AddValueInt32(1), EPropertyBagResult::TypeMismatch);
 
		//Test Struct Set
		TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> StructSetRes = Bag.GetSetRef(StructSetName);
		AITEST_TRUE(TEXT("Bag should have Struct set property"), StructSetRes.IsValid());
		FPropertyBagSetRef StructSet = StructSetRes.GetValue();

		FTestStructHashable1 TestStructInstance1;
		TestStructInstance1.Float = 1.0f;

		FTestStructHashable1 TestStructInstance2;
		TestStructInstance2.Float = 2.0f;

		FTestStructComplex ComplexStructInstance;

		AITEST_EQUAL(TEXT("Adding struct value to set should succeed"), StructSet.AddValueStruct(FConstStructView::Make(TestStructInstance1)), EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("We should be able to find the struct we just added"), StructSet.Contains(FConstStructView::Make(TestStructInstance1).GetMemory()).GetValue());
		AITEST_EQUAL(TEXT("Adding a different struct value to set should succeed"), StructSet.AddValueStruct(FConstStructView::Make(TestStructInstance2)), EPropertyBagResult::Success);
		AITEST_EQUAL(TEXT("Adding the same struct value to set should return a duplicated value error"), StructSet.AddValueStruct(FConstStructView::Make(TestStructInstance2)), EPropertyBagResult::DuplicatedValue);
		AITEST_EQUAL(TEXT("Adding a different struct type should return a type mismatch error"), StructSet.AddValueStruct(FConstStructView::Make(ComplexStructInstance)), EPropertyBagResult::TypeMismatch);
		AITEST_EQUAL(TEXT("Adding a non struct type to a struct set should return a type mismatch error"), StructSet.AddValueInt32(1), EPropertyBagResult::TypeMismatch);

		//Test Object Set
		UBagTestObject1* TestObject1_Instance1 = NewObject<UBagTestObject1>();
		UBagTestObject1* TestObject1_Instance2 = NewObject<UBagTestObject1>();
		UBagTestObject2* TestObject2 = NewObject<UBagTestObject2>();
		UBagTestObject1Derived* TestObject1Derived = NewObject<UBagTestObject1Derived>();

		TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> ObjectSetRes = Bag.GetSetRef(ObjectSetName);
		AITEST_TRUE(TEXT("Bag should have Object set property"), ObjectSetRes.IsValid());
		FPropertyBagSetRef ObjectSet = ObjectSetRes.GetValue();

		AITEST_EQUAL(TEXT("Adding a TestObject1 type instance to set should succeed"), ObjectSet.AddValueObject(TestObject1_Instance1), EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("We should be able to find the object we just added"), ObjectSet.Contains(TestObject1_Instance1).GetValue());
		AITEST_EQUAL(TEXT("Adding a second instance of type TestObject1 to set should succeed"), ObjectSet.AddValueObject(TestObject1_Instance2), EPropertyBagResult::Success);
		AITEST_EQUAL(TEXT("Adding an object type derived from TestObject1 to set should succeed"), ObjectSet.AddValueObject(TestObject1Derived), EPropertyBagResult::Success);
		AITEST_EQUAL(TEXT("Adding a TestObject2 type instance to a TestObject1 type set should return a type mismatch error"), ObjectSet.AddValueObject(TestObject2), EPropertyBagResult::TypeMismatch);

		//Test Class Set
		TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> ClassSetRes = Bag.GetSetRef(ClassSetName);
		AITEST_TRUE(TEXT("Bag should have Object set property"), ClassSetRes.IsValid());
		FPropertyBagSetRef ClassSet = ClassSetRes.GetValue();

		AITEST_EQUAL(TEXT("Adding a class to a class set should succeed"), ClassSet.AddValueClass(UBagTestObject1::StaticClass()), EPropertyBagResult::Success);
		AITEST_TRUE(TEXT("We should be able to find the class we just added"), ClassSet.Contains(UBagTestObject1::StaticClass()).GetValue());
		AITEST_EQUAL(TEXT("Adding a different type to a class set should return a type mismatch error"), ClassSet.AddValueClass(UBagTestObject2::StaticClass()), EPropertyBagResult::TypeMismatch);
		AITEST_EQUAL(TEXT("Adding a derived class type to a class set should succeed"), ClassSet.AddValueClass(UBagTestObject1Derived::StaticClass()), EPropertyBagResult::Success);

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_Sets, "System.StructUtils.PropertyBag.Sets");

struct FTest_Maps : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName NameToFloatMapName(TEXT("NameToFloatMap"));
		static const FName EnumToStructMapName(TEXT("EnumToStructMap"));
		static const FName StringToObjectMapName(TEXT("StringToObjectMap"));
		static const FName ClassToTextMapName(TEXT("ClassToTextMap"));
		static const FName IntToSoftObjectMapName(TEXT("IntToSoftObjectMap"));
		static const FName SoftClassToSoftObjectMapName(TEXT("SoftClassToSoftObjectMap"));
		static const FName StructToSoftClassMapName(TEXT("StructToSoftClassMap"));
		static const FName VectorToClassMapName(TEXT("VectorToClassMap"));

		FInstancedPropertyBag Bag;
		EPropertyBagAlterationResult AlterationResult = Bag.AddProperties({
			{ NameToFloatMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::Float, nullptr, CPF_Edit, EPropertyBagPropertyType::Name },
			{ EnumToStructMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::Struct, FTestStructHashable1::StaticStruct(), CPF_Edit, EPropertyBagPropertyType::Enum, StaticEnum<EPropertyBagTest1>() },
			{ StringToObjectMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::Object, UBagTestObject1::StaticClass(), CPF_Edit, EPropertyBagPropertyType::String },
			{ ClassToTextMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::Text, nullptr, CPF_Edit, EPropertyBagPropertyType::Class, UBagTestObject1::StaticClass() },
			{ IntToSoftObjectMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::SoftObject, UBagTestObject1::StaticClass(), CPF_Edit, EPropertyBagPropertyType::Int32 },
			{ SoftClassToSoftObjectMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::SoftObject, UBagTestObject1::StaticClass(), CPF_Edit, EPropertyBagPropertyType::SoftClass, UBagTestObject2::StaticClass() },
			{ StructToSoftClassMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::SoftClass, UBagTestObject1::StaticClass(), CPF_Edit, EPropertyBagPropertyType::Struct, FTestStructHashable1::StaticStruct() },
		});
		AITEST_EQUAL(TEXT("Bag.AddProperties should succeed"), AlterationResult, EPropertyBagAlterationResult::Success);
		
		{
			AlterationResult = Bag.AddMapProperty(VectorToClassMapName, EPropertyBagPropertyType::Struct, EPropertyBagPropertyType::Class, TBaseStructure<FVector>::Get(), UBagTestObject1::StaticClass());
			AITEST_EQUAL(TEXT("Bag.AddMapProperty should succeed"), AlterationResult, EPropertyBagAlterationResult::Success);
		}

		{
			AlterationResult = Bag.AddMapProperty(FName("BoolKeyTypeIsNotHashable"), EPropertyBagPropertyType::Bool, EPropertyBagPropertyType::Struct, nullptr, FTestStructHashable1::StaticStruct());
			AITEST_EQUAL(TEXT("Bag.AddMapProperty should fail with PropertyKeyTypeNotHashable for Bool KeyType"), AlterationResult, EPropertyBagAlterationResult::PropertyKeyTypeNotHashable);

			AlterationResult = Bag.AddMapProperty(FName("TextKeyTypeIsNotHashable"), EPropertyBagPropertyType::Text, EPropertyBagPropertyType::Struct, nullptr, FTestStructHashable1::StaticStruct());
			AITEST_EQUAL(TEXT("Bag.AddMapProperty should fail with PropertyKeyTypeNotHashable for Text KeyType"), AlterationResult, EPropertyBagAlterationResult::PropertyKeyTypeNotHashable);

			AlterationResult = Bag.AddMapProperty(FName("RotatorKeyTypeIsNotHashable"), EPropertyBagPropertyType::Struct, EPropertyBagPropertyType::Struct, TBaseStructure<FRotator>::Get(), FTestStructHashable1::StaticStruct());
			AITEST_EQUAL(TEXT("Bag.AddMapProperty should fail with PropertyKeyTypeNotHashable for FRotator KeyType"), AlterationResult, EPropertyBagAlterationResult::PropertyKeyTypeNotHashable);
		}

		// map: Name->Float
		{
			TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(NameToFloatMapName);
			AITEST_TRUE(TEXT("Bag should have map: Name->Float"), GetMapResult.IsValid());

			FPropertyBagMapRef NameToFloatMap = GetMapResult.GetValue();
			AITEST_EQUAL(TEXT("NameToFloatMap should be empty"), NameToFloatMap.Num(), 0);

			TValueOrError<bool, EPropertyBagResult> ContainsResult = NameToFloatMap.Contains(FName("NotGonnaFindIt"));
			AITEST_TRUE(TEXT("NameToFloatMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("NameToFloatMap.Contains result should be false"), ContainsResult.GetValue(), false);

			ContainsResult = NameToFloatMap.Contains(42);
			AITEST_TRUE(TEXT("NameToFloatMap.Contains result should have error"), ContainsResult.HasError());
			AITEST_EQUAL(TEXT("NameToFloatMap.Contains result should be TypeMismatch"), ContainsResult.GetError(), EPropertyBagResult::TypeMismatch);

			EPropertyBagResult AddResult = NameToFloatMap.Add(FName("PI"), 3.14f);
			AITEST_EQUAL(TEXT("NameToFloatMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("NameToFloatMap should have 1 pair"), NameToFloatMap.Num(), 1);
			
			AddResult = NameToFloatMap.Add(FName("Wrong"), FName("Type"));
			AITEST_EQUAL(TEXT("NameToFloatMap.Add should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("NameToFloatMap should have 1 pair"), NameToFloatMap.Num(), 1);

			AddResult = NameToFloatMap.Add(FName("PI"), 3.14159f);
			AITEST_EQUAL(TEXT("NameToFloatMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("NameToFloatMap should have 1 pair"), NameToFloatMap.Num(), 1);

			AddResult = NameToFloatMap.Add(FName("Answer"), 42.f);
			AITEST_EQUAL(TEXT("NameToFloatMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("NameToFloatMap should have 2 pairs"), NameToFloatMap.Num(), 2);

			ContainsResult = NameToFloatMap.Contains(FName("Answer"));
			AITEST_TRUE(TEXT("NameToFloatMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("NameToFloatMap.Contains result should be true"), ContainsResult.GetValue(), true);

			TValueOrError<float, EPropertyBagResult> FindResult = NameToFloatMap.Find<FName, float>(FName("PI"));
			AITEST_TRUE(TEXT("NameToFloatMap.Find result should have value"), FindResult.HasValue());
			AITEST_EQUAL(TEXT("NameToFloatMap.Find result should be expected value"), FindResult.GetValue(), 3.14159f);

			FindResult = NameToFloatMap.Find<FName, float>(FName("NotGonnaFindIt"));
			AITEST_TRUE(TEXT("NameToFloatMap.Find result should have error"), FindResult.HasError());
			AITEST_EQUAL(TEXT("NameToFloatMap.Find result should be PropertyNotFound"), FindResult.GetError(), EPropertyBagResult::PropertyNotFound);

			EPropertyBagResult RemoveResult = NameToFloatMap.Remove(FName("Answer"));
			AITEST_EQUAL(TEXT("NameToFloatMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("NameToFloatMap should have 1 pair"), NameToFloatMap.Num(), 1);

			ContainsResult = NameToFloatMap.Contains(FName("Answer"));
			AITEST_TRUE(TEXT("NameToFloatMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("NameToFloatMap.Contains result should be false"), ContainsResult.GetValue(), false);

			NameToFloatMap.Empty();
			AITEST_EQUAL(TEXT("NameToFloatMap should be empty"), NameToFloatMap.Num(), 0);

			ContainsResult = NameToFloatMap.Contains(FName("PI"));
			AITEST_TRUE(TEXT("NameToFloatMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("NameToFloatMap.Contains result should be false"), ContainsResult.GetValue(), false);
		}

		// map: Enum->Struct
		{
			TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(EnumToStructMapName);
			AITEST_TRUE(TEXT("Bag should have map: Enum->Struct"), GetMapResult.IsValid());

			FPropertyBagMapRef EnumToStructMap = GetMapResult.GetValue();
			AITEST_EQUAL(TEXT("EnumToStructMap should be empty"), EnumToStructMap.Num(), 0);

			TValueOrError<bool, EPropertyBagResult> ContainsResult = EnumToStructMap.Contains(EPropertyBagTest1::Foo);
			AITEST_TRUE(TEXT("EnumToStructMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("EnumToStructMap.Contains result should be false"), ContainsResult.GetValue(), false);

			ContainsResult = EnumToStructMap.Contains(EPropertyBagTest2::Bingo);
			AITEST_TRUE(TEXT("EnumToStructMap.Contains result should have error"), ContainsResult.HasError());
			AITEST_EQUAL(TEXT("EnumToStructMap.Contains result should be TypeMismatch"), ContainsResult.GetError(), EPropertyBagResult::TypeMismatch);

			FTestStructHashable1 Struct1;
			Struct1.Float = 3.14f;

			EPropertyBagResult AddResult = EnumToStructMap.Add(EPropertyBagTest1::Foo, Struct1);
			AITEST_EQUAL(TEXT("EnumToStructMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("EnumToStructMap should have 1 pair"), EnumToStructMap.Num(), 1);

			AddResult = EnumToStructMap.Add(EPropertyBagTest1::Bar, FName("WrongType"));
			AITEST_EQUAL(TEXT("EnumToStructMap.Add should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("EnumToStructMap should have 1 pair"), EnumToStructMap.Num(), 1);

			FTestStructHashable1 Struct2;
			Struct2.Float = 3.14159f;

			AddResult = EnumToStructMap.Add(EPropertyBagTest1::Foo, Struct2);
			AITEST_EQUAL(TEXT("EnumToStructMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("EnumToStructMap should have 1 pair"), EnumToStructMap.Num(), 1);

			FTestStructHashable1 Struct3;
			Struct3.Float = 42.f;

			AddResult = EnumToStructMap.Add(EPropertyBagTest1::Bar, Struct3);
			AITEST_EQUAL(TEXT("EnumToStructMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("EnumToStructMap should have 2 pairs"), EnumToStructMap.Num(), 2);

			ContainsResult = EnumToStructMap.Contains(EPropertyBagTest1::Bar);
			AITEST_TRUE(TEXT("EnumToStructMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("EnumToStructMap.Contains result should be true"), ContainsResult.GetValue(), true);

			TValueOrError<FTestStructHashable1, EPropertyBagResult> FindResult = EnumToStructMap.Find<EPropertyBagTest1, FTestStructHashable1>(EPropertyBagTest1::Bar);
			AITEST_TRUE(TEXT("EnumToStructMap.Find result should have value"), FindResult.HasValue());
			AITEST_EQUAL(TEXT("EnumToStructMap.Find result should be expected value"), FindResult.GetValue().Float, 42.f);

			FindResult = EnumToStructMap.Find<EPropertyBagTest1, FTestStructHashable1>(EPropertyBagTest1::Baz);
			AITEST_TRUE(TEXT("EnumToStructMap.Find result should have error"), FindResult.HasError());
			AITEST_EQUAL(TEXT("EnumToStructMap.Find result should be PropertyNotFound"), FindResult.GetError(), EPropertyBagResult::PropertyNotFound);

			EPropertyBagResult RemoveResult = EnumToStructMap.Remove(EPropertyBagTest1::Bar);
			AITEST_EQUAL(TEXT("EnumToStructMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("EnumToStructMap should have 1 pair"), EnumToStructMap.Num(), 1);

			ContainsResult = EnumToStructMap.Contains(EPropertyBagTest1::Bar);
			AITEST_TRUE(TEXT("EnumToStructMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("EnumToStructMap.Contains result should be false"), ContainsResult.GetValue(), false);

			EnumToStructMap.Empty();
			AITEST_EQUAL(TEXT("EnumToStructMap should be empty"), EnumToStructMap.Num(), 0);

			ContainsResult = EnumToStructMap.Contains(EPropertyBagTest1::Foo);
			AITEST_TRUE(TEXT("EnumToStructMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("EnumToStructMap.Contains result should be false"), ContainsResult.GetValue(), false);
		}

		// map: String->Object
		{
			TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(StringToObjectMapName);
			AITEST_TRUE(TEXT("Bag should have map: String->Object"), GetMapResult.IsValid());

			FPropertyBagMapRef StringToObjectMap = GetMapResult.GetValue();
			AITEST_EQUAL(TEXT("StringToObjectMap should be empty"), StringToObjectMap.Num(), 0);

			TValueOrError<bool, EPropertyBagResult> ContainsResult = StringToObjectMap.Contains(FString(TEXT("Key")));
			AITEST_TRUE(TEXT("StringToObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("StringToObjectMap.Contains result should be false"), ContainsResult.GetValue(), false);

			UBagTestObject1* TestObject1_Instance1 = NewObject<UBagTestObject1>();

			EPropertyBagResult AddResult = StringToObjectMap.Add(FString(TEXT("Key")), TestObject1_Instance1);
			AITEST_EQUAL(TEXT("StringToObjectMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("StringToObjectMap should have 1 pair"), StringToObjectMap.Num(), 1);

			UBagTestObject2* TestObject2_Instance1 = NewObject<UBagTestObject2>();

			AddResult = StringToObjectMap.Add(FString(TEXT("AnotherKey")), TestObject2_Instance1);
			AITEST_EQUAL(TEXT("StringToObjectMap.Add should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("StringToObjectMap should have 1 pair"), StringToObjectMap.Num(), 1);

			UBagTestObject1* TestObject1_Instance2 = NewObject<UBagTestObject1>();

			AddResult = StringToObjectMap.Add(FString(TEXT("Key")), TestObject1_Instance2);
			AITEST_EQUAL(TEXT("StringToObjectMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("StringToObjectMap should have 1 pair"), StringToObjectMap.Num(), 1);

			UBagTestObject1* TestObject1_Instance3 = NewObject<UBagTestObject1>();

			AddResult = StringToObjectMap.Add(FString(TEXT("AnotherKey")), TestObject1_Instance3);
			AITEST_EQUAL(TEXT("StringToObjectMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("StringToObjectMap should have 2 pairs"), StringToObjectMap.Num(), 2);

			ContainsResult = StringToObjectMap.Contains(FString(TEXT("AnotherKey")));
			AITEST_TRUE(TEXT("StringToObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("StringToObjectMap.Contains result should be true"), ContainsResult.GetValue(), true);

			TValueOrError<UBagTestObject1*, EPropertyBagResult> FindResult = StringToObjectMap.Find<FString, UBagTestObject1*>(FString(TEXT("AnotherKey")));
			AITEST_TRUE(TEXT("StringToObjectMap.Find result should have value"), FindResult.HasValue());
			AITEST_EQUAL(TEXT("StringToObjectMap.Find result should be expected value"), FindResult.GetValue(), TestObject1_Instance3);

			FindResult = StringToObjectMap.Find<FString, UBagTestObject1*>(FString(TEXT("MissingKey")));
			AITEST_TRUE(TEXT("StringToObjectMap.Find result should have error"), FindResult.HasError());
			AITEST_EQUAL(TEXT("StringToObjectMap.Find result should be PropertyNotFound"), FindResult.GetError(), EPropertyBagResult::PropertyNotFound);

			EPropertyBagResult RemoveResult = StringToObjectMap.Remove(FString(TEXT("AnotherKey")));
			AITEST_EQUAL(TEXT("StringToObjectMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("StringToObjectMap should have 1 pair"), StringToObjectMap.Num(), 1);

			ContainsResult = StringToObjectMap.Contains(FString(TEXT("AnotherKey")));
			AITEST_TRUE(TEXT("StringToObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("StringToObjectMap.Contains result should be false"), ContainsResult.GetValue(), false);

			StringToObjectMap.Empty();
			AITEST_EQUAL(TEXT("StringToObjectMap should be empty"), StringToObjectMap.Num(), 0);

			ContainsResult = StringToObjectMap.Contains(FString(TEXT("Key")));
			AITEST_TRUE(TEXT("StringToObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("StringToObjectMap.Contains result should be false"), ContainsResult.GetValue(), false);
		}

		// map: Class->Text
		{
			TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(ClassToTextMapName);
			AITEST_TRUE(TEXT("Bag should have map: Class->Text"), GetMapResult.IsValid());

			FPropertyBagMapRef ClassToTextMap = GetMapResult.GetValue();
			AITEST_EQUAL(TEXT("ClassToTextMap should be empty"), ClassToTextMap.Num(), 0);

			TValueOrError<bool, EPropertyBagResult> ContainsResult = ClassToTextMap.Contains(UBagTestObject1::StaticClass());
			AITEST_TRUE(TEXT("ClassToTextMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("ClassToTextMap.Contains result should be false"), ContainsResult.GetValue(), false);

			EPropertyBagResult AddResult = ClassToTextMap.Add(UBagTestObject1::StaticClass(), FText::FromString(TEXT("Value")));
			AITEST_EQUAL(TEXT("ClassToTextMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("ClassToTextMap should have 1 pair"), ClassToTextMap.Num(), 1);

			AddResult = ClassToTextMap.Add(UBagTestObject2::StaticClass(), FText::FromString(TEXT("WrongKeyType")));
			AITEST_EQUAL(TEXT("ClassToTextMap.Add should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("ClassToTextMap should have 1 pair"), ClassToTextMap.Num(), 1);

			AddResult = ClassToTextMap.Add(UBagTestObject1::StaticClass(), FText::FromString(TEXT("AnotherValue")));
			AITEST_EQUAL(TEXT("ClassToTextMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("ClassToTextMap should have 1 pair"), ClassToTextMap.Num(), 1);

			AddResult = ClassToTextMap.Add(UBagTestObject1Derived::StaticClass(), FText::FromString(TEXT("ValueForSecondPair")));
			AITEST_EQUAL(TEXT("ClassToTextMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("ClassToTextMap should have 2 pairs"), ClassToTextMap.Num(), 2);

			ContainsResult = ClassToTextMap.Contains(UBagTestObject1Derived::StaticClass());
			AITEST_TRUE(TEXT("ClassToTextMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("ClassToTextMap.Contains result should be true"), ContainsResult.GetValue(), true);

			TValueOrError<FText, EPropertyBagResult> FindResult = ClassToTextMap.Find<UClass*, FText>(UBagTestObject1Derived::StaticClass());
			AITEST_TRUE(TEXT("ClassToTextMap.Find result should have value"), FindResult.HasValue());
			AITEST_EQUAL(TEXT("ClassToTextMap.Find result should be expected value"), FindResult.GetValue(), FText::FromString(TEXT("ValueForSecondPair")));

			FindResult = ClassToTextMap.Find<UClass*, FText>(UBagTestObject1Derived2::StaticClass());
			AITEST_TRUE(TEXT("ClassToTextMap.Find result should have error"), FindResult.HasError());
			AITEST_EQUAL(TEXT("ClassToTextMap.Find result should be PropertyNotFound"), FindResult.GetError(), EPropertyBagResult::PropertyNotFound);

			EPropertyBagResult RemoveResult = ClassToTextMap.Remove(UBagTestObject1Derived::StaticClass());
			AITEST_EQUAL(TEXT("ClassToTextMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("ClassToTextMap should have 1 pair"), ClassToTextMap.Num(), 1);

			ContainsResult = ClassToTextMap.Contains(UBagTestObject1Derived::StaticClass());
			AITEST_TRUE(TEXT("ClassToTextMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("ClassToTextMap.Contains result should be false"), ContainsResult.GetValue(), false);

			ClassToTextMap.Empty();
			AITEST_EQUAL(TEXT("ClassToTextMap should be empty"), ClassToTextMap.Num(), 0);

			ContainsResult = ClassToTextMap.Contains(UBagTestObject1::StaticClass());
			AITEST_TRUE(TEXT("ClassToTextMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("ClassToTextMap.Contains result should be false"), ContainsResult.GetValue(), false);
		}

		// map: int->SoftObject
		{
			TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(IntToSoftObjectMapName);
			AITEST_TRUE(TEXT("Bag should have map: int->SoftObject"), GetMapResult.IsValid());

			FPropertyBagMapRef IntToSoftObjectMap = GetMapResult.GetValue();
			AITEST_EQUAL(TEXT("IntToSoftObjectMap should be empty"), IntToSoftObjectMap.Num(), 0);

			TValueOrError<bool, EPropertyBagResult> ContainsResult = IntToSoftObjectMap.Contains(42);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Contains result should be false"), ContainsResult.GetValue(), false);

			EPropertyBagResult AddResult = IntToSoftObjectMap.Add(42, FSoftObjectPath("/Game/Answer"));
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Add<FSoftObjectPath> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("IntToSoftObjectMap should have 1 pair"), IntToSoftObjectMap.Num(), 1);

			UBagTestObject2* TestObject2_Instance1 = NewObject<UBagTestObject2>();

			AddResult = IntToSoftObjectMap.Add(13, TestObject2_Instance1);
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Add<UBagTestObject2> should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("IntToSoftObjectMap should have 1 pair"), IntToSoftObjectMap.Num(), 1);
			
			UBagTestObject1* TestObject1_Instance1 = NewObject<UBagTestObject1>();

			AddResult = IntToSoftObjectMap.Add(128, TestObject1_Instance1);
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Add<UBagTestObject1> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("IntToSoftObjectMap should have 2 pairs"), IntToSoftObjectMap.Num(), 2);

			UBagTestObject1* TestObject1_Instance2 = NewObject<UBagTestObject1>();

			AddResult = IntToSoftObjectMap.Add(13, FSoftObjectPath(TestObject1_Instance2));
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Add<FSoftObjectPath> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("IntToSoftObjectMap should have 3 pairs"), IntToSoftObjectMap.Num(), 3);
						
			ContainsResult = IntToSoftObjectMap.Contains(42);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Contains result should be true"), ContainsResult.GetValue(), true);

			TValueOrError<FSoftObjectPath, EPropertyBagResult> FindAsSoftObjectPathResult = IntToSoftObjectMap.Find<int, FSoftObjectPath>(13);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Find<FSoftObjectPath> result should have value"), FindAsSoftObjectPathResult.HasValue());
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Find<FSoftObjectPath> result should be expected value"), FindAsSoftObjectPathResult.GetValue(), FSoftObjectPath(TestObject1_Instance2));

			TValueOrError<FSoftObjectPtr, EPropertyBagResult> FindAsSoftObjectPtrResult = IntToSoftObjectMap.Find<int, FSoftObjectPtr>(13);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Find<FSoftObjectPtr> result should have value"), FindAsSoftObjectPtrResult.HasValue());
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Find<FSoftObjectPtr> result should be expected value"), FindAsSoftObjectPtrResult.GetValue(), FSoftObjectPtr(TestObject1_Instance2));

			TValueOrError<UBagTestObject1*, EPropertyBagResult> FindAsUObjectPtrResult = IntToSoftObjectMap.Find<int, UBagTestObject1*>(13);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Find<UBagTestObject1> result should have value"), FindAsUObjectPtrResult.HasValue());
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Find<UBagTestObject1> result should be expected value"), FindAsUObjectPtrResult.GetValue(), TestObject1_Instance2);

			FindAsSoftObjectPathResult = IntToSoftObjectMap.Find<int, FSoftObjectPath>(42);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Find<FSoftObjectPath> result should have value"), FindAsSoftObjectPathResult.HasValue());
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Find<FSoftObjectPath> result should be expected value"), FindAsSoftObjectPathResult.GetValue(), FSoftObjectPath("/Game/Answer"));

			FindAsSoftObjectPtrResult = IntToSoftObjectMap.Find<int, FSoftObjectPtr>(42);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Find<FSoftObjectPtr> result should have value"), FindAsSoftObjectPtrResult.HasValue());
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Find<FSoftObjectPtr> result should be expected value"), FindAsSoftObjectPtrResult.GetValue(), FSoftObjectPtr(FSoftObjectPath("/Game/Answer")));

			FindAsUObjectPtrResult = IntToSoftObjectMap.Find<int, UBagTestObject1*>(42);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Find<UBagTestObject1> result should have value"), FindAsUObjectPtrResult.HasValue());
			AITEST_NULL(TEXT("IntToSoftObjectMap.Find<UBagTestObject1> result should be null"), FindAsUObjectPtrResult.GetValue());

			EPropertyBagResult RemoveResult = IntToSoftObjectMap.Remove(128);
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("IntToSoftObjectMap should have 2 pair"), IntToSoftObjectMap.Num(), 2);

			ContainsResult = IntToSoftObjectMap.Contains(128);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Contains result should be false"), ContainsResult.GetValue(), false);

			IntToSoftObjectMap.Empty();
			AITEST_EQUAL(TEXT("IntToSoftObjectMap should be empty"), IntToSoftObjectMap.Num(), 0);

			ContainsResult = IntToSoftObjectMap.Contains(42);
			AITEST_TRUE(TEXT("IntToSoftObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("IntToSoftObjectMap.Contains result should be false"), ContainsResult.GetValue(), false);
		}

		// map: SoftClass->SoftObject
		{
			TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(SoftClassToSoftObjectMapName);
			AITEST_TRUE(TEXT("Bag should have map: SoftClass->SoftObject"), GetMapResult.IsValid());

			FPropertyBagMapRef SoftClassToSoftObjectMap = GetMapResult.GetValue();
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap should be empty"), SoftClassToSoftObjectMap.Num(), 0);

			TValueOrError<bool, EPropertyBagResult> ContainsResult = SoftClassToSoftObjectMap.Contains(FSoftClassPath("/Game/Classy"));
			AITEST_TRUE(TEXT("SoftClassToSoftObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Contains result should be false"), ContainsResult.GetValue(), false);

			EPropertyBagResult AddResult = SoftClassToSoftObjectMap.Add(FSoftClassPath("/Game/Classy"), FSoftObjectPath("/Game/Answer"));
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Add<FSoftClassPath, FSoftObjectPath> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap should have 1 pair"), SoftClassToSoftObjectMap.Num(), 1);

			ContainsResult = SoftClassToSoftObjectMap.Contains(FSoftClassPath("/Game/Classy"));
			AITEST_TRUE(TEXT("SoftClassToSoftObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Contains result should be true"), ContainsResult.GetValue(), true);

			UBagTestObject1* TestObject1_Instance1 = NewObject<UBagTestObject1>();

			AddResult = SoftClassToSoftObjectMap.Add(UBagTestObject2::StaticClass(), FSoftObjectPath(TestObject1_Instance1));
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Add<FSoftClassPath, FSoftObjectPath> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap should have 2 pairs"), SoftClassToSoftObjectMap.Num(), 2);

			ContainsResult = SoftClassToSoftObjectMap.Contains(FSoftClassPath(UBagTestObject2::StaticClass()));
			AITEST_TRUE(TEXT("SoftClassToSoftObjectMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Contains result should be true"), ContainsResult.GetValue(), true);

			AddResult = SoftClassToSoftObjectMap.Add(UBagTestObject1::StaticClass(), FSoftObjectPath("/Game/Mismatch"));
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Add<UBagTestObject2> should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap should have 2 pairs"), SoftClassToSoftObjectMap.Num(), 2);

			TValueOrError<FSoftObjectPath, EPropertyBagResult> FindAsSoftObjectPathResult = SoftClassToSoftObjectMap.Find<FSoftClassPath, FSoftObjectPath>(FSoftClassPath(UBagTestObject2::StaticClass()));
			AITEST_TRUE(TEXT("SoftClassToSoftObjectMap.Find<FSoftClassPath, FSoftObjectPath> result should have value"), FindAsSoftObjectPathResult.HasValue());
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Find<FSoftClassPath, FSoftObjectPath> result should be expected value"), FindAsSoftObjectPathResult.GetValue(), FSoftObjectPath(TestObject1_Instance1));

			FindAsSoftObjectPathResult = SoftClassToSoftObjectMap.Find<UClass*, FSoftObjectPath>(UBagTestObject2::StaticClass());
			AITEST_TRUE(TEXT("SoftClassToSoftObjectMap.Find<UClass*, FSoftObjectPath> result should have value"), FindAsSoftObjectPathResult.HasValue());
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Find<UClass*, FSoftObjectPath> result should be expected value"), FindAsSoftObjectPathResult.GetValue(), FSoftObjectPath(TestObject1_Instance1));

			TValueOrError<UBagTestObject1*, EPropertyBagResult> FindAsUObjectPtrResult = SoftClassToSoftObjectMap.Find<FSoftClassPath, UBagTestObject1*>(FSoftClassPath(UBagTestObject2::StaticClass()));
			AITEST_TRUE(TEXT("SoftClassToSoftObjectMap.Find<FSoftClassPath, UBagTestObject1*> result should have value"), FindAsUObjectPtrResult.HasValue());
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Find<FSoftClassPath, UBagTestObject1*> result should be expected value"), FindAsUObjectPtrResult.GetValue(), TestObject1_Instance1);

			FindAsUObjectPtrResult = SoftClassToSoftObjectMap.Find<UClass*, UBagTestObject1*>(UBagTestObject2::StaticClass());
			AITEST_TRUE(TEXT("SoftClassToSoftObjectMap.Find<UClass*, UBagTestObject1*> result should have value"), FindAsUObjectPtrResult.HasValue());
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Find<UClass*, UBagTestObject1*> result should be expected value"), FindAsUObjectPtrResult.GetValue(), TestObject1_Instance1);

			EPropertyBagResult RemoveResult = SoftClassToSoftObjectMap.Remove(FSoftClassPath("/Game/Classy"));
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap should have 1 pair"), SoftClassToSoftObjectMap.Num(), 1);

			RemoveResult = SoftClassToSoftObjectMap.Remove(UBagTestObject2::StaticClass());
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("SoftClassToSoftObjectMap should be empty"), SoftClassToSoftObjectMap.Num(), 0);
		}

		// map: Struct->SoftClass
		{
			TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(StructToSoftClassMapName);
			AITEST_TRUE(TEXT("Bag should have map: Struct->SoftClass"), GetMapResult.IsValid());

			FPropertyBagMapRef StructToSoftClassMap = GetMapResult.GetValue();
			AITEST_EQUAL(TEXT("StructToSoftClassMap should be empty"), StructToSoftClassMap.Num(), 0);
			
			FTestStructHashable1 Struct1;
			Struct1.Float = 3.14f;

			TValueOrError<bool, EPropertyBagResult> ContainsResult = StructToSoftClassMap.Contains(Struct1);
			AITEST_TRUE(TEXT("StructToSoftClassMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Contains result should be false"), ContainsResult.GetValue(), false);

			EPropertyBagResult AddResult = StructToSoftClassMap.Add(Struct1, FSoftClassPath("/Game/Classy"));
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Add<FSoftClassPath> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("StructToSoftClassMap should have 1 pair"), StructToSoftClassMap.Num(), 1);

			ContainsResult = StructToSoftClassMap.Contains(Struct1);
			AITEST_TRUE(TEXT("StructToSoftClassMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Contains result should be true"), ContainsResult.GetValue(), true);

			FTestStructHashable1 Struct2;
			Struct2.Float = 3.14159f;

			AddResult = StructToSoftClassMap.Add(Struct2, UBagTestObject1::StaticClass());
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Add<UClass*> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("StructToSoftClassMap should have 2 pairs"), StructToSoftClassMap.Num(), 2);

			ContainsResult = StructToSoftClassMap.Contains(Struct2);
			AITEST_TRUE(TEXT("StructToSoftClassMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Contains result should be true"), ContainsResult.GetValue(), true);

			FTestStructHashable2 WrongStructType;
			WrongStructType.Int = 42;

			AddResult = StructToSoftClassMap.Add(WrongStructType, FSoftClassPath("/Game/Mismatch"));
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Add<FSoftClassPath> should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("StructToSoftClassMap should have 2 pairs"), StructToSoftClassMap.Num(), 2);

			TValueOrError<UClass*, EPropertyBagResult> FindAsUClassPtrResult = StructToSoftClassMap.Find<FTestStructHashable1, UClass*>(Struct1);
			AITEST_TRUE(TEXT("StructToSoftClassMap.Find<FTestStructHashable1, UClass*> result should have value"), FindAsUClassPtrResult.HasValue());
			AITEST_NULL(TEXT("StructToSoftClassMap.Find<FTestStructHashable1, UClass*> result should be null"), FindAsUClassPtrResult.GetValue());

			TValueOrError<FSoftClassPath, EPropertyBagResult> FindAsSoftClassPathResult = StructToSoftClassMap.Find<FTestStructHashable1, FSoftClassPath>(Struct1);
			AITEST_TRUE(TEXT("StructToSoftClassMap.Find<FTestStructHashable1, FSoftClassPath> result should have value"), FindAsSoftClassPathResult.HasValue());
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Find<FTestStructHashable1, FSoftClassPath> result should be expected value"), FindAsSoftClassPathResult.GetValue(), FSoftClassPath("/Game/Classy"));

			FindAsUClassPtrResult = StructToSoftClassMap.Find<FTestStructHashable1, UClass*>(Struct2);
			AITEST_TRUE(TEXT("StructToSoftClassMap.Find<FTestStructHashable1, UClass*> result should have value"), FindAsUClassPtrResult.HasValue());
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Find<FTestStructHashable1, UClass*> result should be expected value"), FindAsUClassPtrResult.GetValue(), UBagTestObject1::StaticClass());

			FindAsSoftClassPathResult = StructToSoftClassMap.Find<FTestStructHashable1, FSoftClassPath>(Struct2);
			AITEST_TRUE(TEXT("StructToSoftClassMap.Find<FTestStructHashable1, FSoftClassPath> result should have value"), FindAsSoftClassPathResult.HasValue());
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Find<FTestStructHashable1, FSoftClassPath> result should be expected value"), FindAsSoftClassPathResult.GetValue(), FSoftClassPath(UBagTestObject1::StaticClass()));

			EPropertyBagResult RemoveResult = StructToSoftClassMap.Remove(Struct2);
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("StructToSoftClassMap should have 1 pair"), StructToSoftClassMap.Num(), 1);

			RemoveResult = StructToSoftClassMap.Remove(Struct1);
			AITEST_EQUAL(TEXT("StructToSoftClassMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("StructToSoftClassMap should be empty"), StructToSoftClassMap.Num(), 0);
		}

		// map: Vector->Class
		{
			TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(VectorToClassMapName);
			AITEST_TRUE(TEXT("Bag should have map: Vector->Class"), GetMapResult.IsValid());

			FPropertyBagMapRef VectorToClassMap = GetMapResult.GetValue();
			AITEST_EQUAL(TEXT("VectorToClassMap should be empty"), VectorToClassMap.Num(), 0);

			TValueOrError<bool, EPropertyBagResult> ContainsResult = VectorToClassMap.Contains(FVector{ 1.0, 2.0, 3.0 });
			AITEST_TRUE(TEXT("VectorToClassMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("VectorToClassMap.Contains result should be false"), ContainsResult.GetValue(), false);

			EPropertyBagResult AddResult = VectorToClassMap.Add(FVector{ 1.0, 2.0, 3.0 }, UBagTestObject1::StaticClass());
			AITEST_EQUAL(TEXT("VectorToClassMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("VectorToClassMap should have 1 pair"), VectorToClassMap.Num(), 1);

			ContainsResult = VectorToClassMap.Contains(FVector{ 1.0, 2.0, 3.0 });
			AITEST_TRUE(TEXT("VectorToClassMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("VectorToClassMap.Contains result should be true"), ContainsResult.GetValue(), true);

			AddResult = VectorToClassMap.Add(FVector{ 4.0, 5.0, 6.0 }, UBagTestObject1Derived::StaticClass());
			AITEST_EQUAL(TEXT("VectorToClassMap.Add should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("VectorToClassMap should have 2 pairs"), VectorToClassMap.Num(), 2);

			ContainsResult = VectorToClassMap.Contains(FVector{ 4.0, 5.0, 6.0 });
			AITEST_TRUE(TEXT("VectorToClassMap.Contains result should have value"), ContainsResult.HasValue());
			AITEST_EQUAL(TEXT("VectorToClassMap.Contains result should be true"), ContainsResult.GetValue(), true);

			TValueOrError<UClass*, EPropertyBagResult> FindResult = VectorToClassMap.Find<FVector, UClass*>(FVector{ 1.0, 2.0, 3.0 });
			AITEST_TRUE(TEXT("VectorToClassMap.Find result should have value"), FindResult.HasValue());
			AITEST_EQUAL(TEXT("VectorToClassMap.Find result should be expected value"), FindResult.GetValue(), UBagTestObject1::StaticClass());

			FindResult = VectorToClassMap.Find<FVector, UClass*>(FVector{ 4.0, 5.0, 6.0 });
			AITEST_TRUE(TEXT("VectorToClassMap.Find result should have value"), FindResult.HasValue());
			AITEST_EQUAL(TEXT("VectorToClassMap.Find result should be expected value"), FindResult.GetValue(), UBagTestObject1Derived::StaticClass());

			EPropertyBagResult RemoveResult = VectorToClassMap.Remove(FVector{ 1.0, 2.0, 3.0 });
			AITEST_EQUAL(TEXT("VectorToClassMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("VectorToClassMap should have 1 pair"), VectorToClassMap.Num(), 1);

			RemoveResult = VectorToClassMap.Remove(FVector{ 1.0, 2.0, 3.0 });
			AITEST_EQUAL(TEXT("VectorToClassMap.Remove should fail"), RemoveResult, EPropertyBagResult::PropertyNotFound);
			AITEST_EQUAL(TEXT("VectorToClassMap should have 1 pair"), VectorToClassMap.Num(), 1);

			RemoveResult = VectorToClassMap.Remove(FVector{ 4.0, 5.0, 6.0 });
			AITEST_EQUAL(TEXT("VectorToClassMap.Remove should succeed"), RemoveResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("VectorToClassMap should be empty"), VectorToClassMap.Num(), 0);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_Maps, "System.StructUtils.PropertyBag.Maps");

struct FTest_MapsSerialize : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName NameToFloatMapName(TEXT("NameToFloatMap"));
		static const FName EnumToStructMapName(TEXT("EnumToStructMap"));
		static const FName VectorToClassMapName(TEXT("VectorToClassMap"));

		FInstancedPropertyBag Bag;
		Bag.AddProperties({
			{ NameToFloatMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::Float, nullptr, CPF_Edit, EPropertyBagPropertyType::Name },
			{ EnumToStructMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::Struct, FTestStructHashable1::StaticStruct(), CPF_Edit, EPropertyBagPropertyType::Enum, StaticEnum<EPropertyBagTest1>() },
			{ VectorToClassMapName, EPropertyBagContainerType::Map, EPropertyBagPropertyType::Class, UBagTestObject1::StaticClass(), CPF_Edit, EPropertyBagPropertyType::Struct, TBaseStructure<FVector>::Get() },
		});

		TArray<uint8> Memory;

		FMemoryWriter Writer(Memory);
		FObjectAndNameAsStringProxyArchive WriterProxy(Writer, /*bInLoadIfFindFails*/false);
		const bool bSaveResult = Bag.Serialize(WriterProxy);
		AITEST_TRUE(TEXT("Saving should succeed"), bSaveResult);

		FInstancedPropertyBag Bag2;
		FMemoryReader Reader(Memory);
		FObjectAndNameAsStringProxyArchive ReaderProxy(Reader, /*bInLoadIfFindFails*/true);
		const bool bLoadResult = Bag2.Serialize(ReaderProxy);
		AITEST_TRUE(TEXT("Loading to Bag2 should succeed"), bLoadResult);
		AITEST_EQUAL(TEXT("Bag2 should have 3 items"), Bag2.GetNumPropertiesInBag(), 3);

		{
			const FPropertyBagPropertyDesc* NameToFloatMapPropertyDesc = Bag2.FindPropertyDescByName(NameToFloatMapName);
			AITEST_NOT_NULL(TEXT("NameToFloatMapPropertyDesc should be not null"), NameToFloatMapPropertyDesc);
			AITEST_EQUAL(TEXT("NameToFloatMapPropertyDesc->KeyType should be Name"), NameToFloatMapPropertyDesc->KeyType, EPropertyBagPropertyType::Name);
			AITEST_NULL(TEXT("NameToFloatMapPropertyDesc->KeyTypeObject should be null"), NameToFloatMapPropertyDesc->KeyTypeObject);
			AITEST_EQUAL(TEXT("NameToFloatMapPropertyDesc->ValueType should be Float"), NameToFloatMapPropertyDesc->ValueType, EPropertyBagPropertyType::Float);
			AITEST_NULL(TEXT("NameToFloatMapPropertyDesc->ValueTypeObject should be null"), NameToFloatMapPropertyDesc->ValueTypeObject);
			AITEST_EQUAL(TEXT("NameToFloatMapPropertyDesc->ContainerTypes should have 1 type"), NameToFloatMapPropertyDesc->ContainerTypes.Num(), 1);
			AITEST_EQUAL(TEXT("NameToFloatMapPropertyDesc->ContainerTypes.GetFirstContainerType() should be Map"), NameToFloatMapPropertyDesc->ContainerTypes.GetFirstContainerType(), EPropertyBagContainerType::Map);
		}

		{
			const FPropertyBagPropertyDesc* EnumToStructMapPropertyDesc = Bag2.FindPropertyDescByName(EnumToStructMapName);
			AITEST_NOT_NULL(TEXT("EnumToStructMapPropertyDesc should be not null"), EnumToStructMapPropertyDesc);
			AITEST_EQUAL(TEXT("EnumToStructMapPropertyDesc->KeyType should be Enum"), EnumToStructMapPropertyDesc->KeyType, EPropertyBagPropertyType::Enum);
			AITEST_EQUAL(TEXT("EnumToStructMapPropertyDesc->KeyTypeObject should be EPropertyBagTest1"), Cast<UEnum>(EnumToStructMapPropertyDesc->KeyTypeObject), StaticEnum<EPropertyBagTest1>());
			AITEST_EQUAL(TEXT("EnumToStructMapPropertyDesc->ValueType should be Struct"), EnumToStructMapPropertyDesc->ValueType, EPropertyBagPropertyType::Struct);
			AITEST_EQUAL(TEXT("EnumToStructMapPropertyDesc->ValueTypeObject should be FTestStructHashable1"), Cast<UScriptStruct>(EnumToStructMapPropertyDesc->ValueTypeObject), FTestStructHashable1::StaticStruct());
			AITEST_EQUAL(TEXT("EnumToStructMapPropertyDesc->ContainerTypes should have 1 type"), EnumToStructMapPropertyDesc->ContainerTypes.Num(), 1);
			AITEST_EQUAL(TEXT("EnumToStructMapPropertyDesc->ContainerTypes.GetFirstContainerType() should be Map"), EnumToStructMapPropertyDesc->ContainerTypes.GetFirstContainerType(), EPropertyBagContainerType::Map);
		}
		
		{
			const FPropertyBagPropertyDesc* VectorToClassMapPropertyDesc = Bag2.FindPropertyDescByName(VectorToClassMapName);
			AITEST_NOT_NULL(TEXT("VectorToClassMapPropertyDesc should be not null"), VectorToClassMapPropertyDesc);
			AITEST_EQUAL(TEXT("VectorToClassMapPropertyDesc->KeyType should be Struct"), VectorToClassMapPropertyDesc->KeyType, EPropertyBagPropertyType::Struct);
			AITEST_EQUAL(TEXT("VectorToClassMapPropertyDesc->KeyTypeObject should be FVector"), Cast<UScriptStruct>(VectorToClassMapPropertyDesc->KeyTypeObject), TBaseStructure<FVector>::Get());
			AITEST_EQUAL(TEXT("VectorToClassMapPropertyDesc->ValueType should be Class"), VectorToClassMapPropertyDesc->ValueType, EPropertyBagPropertyType::Class);
			AITEST_EQUAL(TEXT("VectorToClassMapPropertyDesc->ValueTypeObject should be UBagTestObject1"), Cast<UClass>(VectorToClassMapPropertyDesc->ValueTypeObject), UBagTestObject1::StaticClass());
			AITEST_EQUAL(TEXT("VectorToClassMapPropertyDesc->ContainerTypes should have 1 type"), VectorToClassMapPropertyDesc->ContainerTypes.Num(), 1);
			AITEST_EQUAL(TEXT("VectorToClassMapPropertyDesc->ContainerTypes.GetFirstContainerType() should be Map"), VectorToClassMapPropertyDesc->ContainerTypes.GetFirstContainerType(), EPropertyBagContainerType::Map);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_MapsSerialize, "System.StructUtils.PropertyBag.MapsSerialize");

struct FTest_MapsNestedArray : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName NameToIntArrayMapName(TEXT("NameToIntArrayMap"));
		static const FName NestedArrayName(TEXT("NestedIntArray"));
		static const TArray<int32> TestArray = {42, 13, 57};

		FInstancedPropertyBag Bag;

		{
			EPropertyBagAlterationResult AlterationResult = Bag.AddMapProperty(NameToIntArrayMapName, EPropertyBagPropertyType::Name, EPropertyBagPropertyType::Int32, nullptr, nullptr, EPropertyBagContainerType::Array);
			AITEST_EQUAL(TEXT("Bag.AddMapProperty should succeed"), AlterationResult, EPropertyBagAlterationResult::Success);
		}

		TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(NameToIntArrayMapName);
		AITEST_TRUE(TEXT("Bag should have map: Name->IntArray"), GetMapResult.IsValid());

		FPropertyBagMapRef NameToIntArrayMap = GetMapResult.GetValue();
		AITEST_EQUAL(TEXT("NameToIntArrayMap should be empty"), NameToIntArrayMap.Num(), 0);

		// Verify that we can't add values directly to the map
		{
			EPropertyBagResult AddResult = NameToIntArrayMap.Add(FName("NotAnArray"), 42);
			AITEST_EQUAL(TEXT("NameToIntArrayMap.Add<FName,int> should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("NameToIntArrayMap should be empty"), NameToIntArrayMap.Num(), 0);

			AddResult = NameToIntArrayMap.Add(FName("AnIntArray"), TArray<int32>{ 1, 2, 3 });
			AITEST_EQUAL(TEXT("NameToIntArrayMap.Add<FName,TArray<int32>> should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("NameToIntArrayMap should be empty"), NameToIntArrayMap.Num(), 0);
		}
		
		// Reserve a pair for a given key
		{
			EPropertyBagResult AddResult = NameToIntArrayMap.Add(NestedArrayName);
			AITEST_EQUAL(TEXT("NameToIntArrayMap.Add<FName> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("NameToIntArrayMap should have 1 pair"), NameToIntArrayMap.Num(), 1);
			
			AddResult = NameToIntArrayMap.Add(NestedArrayName);
			AITEST_EQUAL(TEXT("NameToIntArrayMap.Add<FName> should fail with DuplicatedValue"), AddResult, EPropertyBagResult::DuplicatedValue);
			AITEST_EQUAL(TEXT("NameToIntArrayMap should have 1 pair"), NameToIntArrayMap.Num(), 1);
		}

		// Verify that we can't get helpers for other nested container types
		{
			TValueOrError<FPropertyBagSetRef, EPropertyBagResult> GetMutableNestedSetResult = NameToIntArrayMap.GetMutableNestedSetRef(NestedArrayName);
			AITEST_TRUE(TEXT("NameToIntArrayMap.GetMutableNestedSetRef<FName>"), GetMutableNestedSetResult.HasError());
			AITEST_EQUAL(TEXT("NameToIntArrayMap.GetMutableNestedSetRef<FName> should fail with TypeMismatch"), GetMutableNestedSetResult.GetError(), EPropertyBagResult::TypeMismatch);
			
			TValueOrError<FPropertyBagMapRef, EPropertyBagResult> GetMutableNestedMapResult = NameToIntArrayMap.GetMutableNestedMapRef(NestedArrayName);
			AITEST_TRUE(TEXT("NameToIntArrayMap.GetMutableNestedMapRef<FName>"), GetMutableNestedMapResult.HasError());
			AITEST_EQUAL(TEXT("NameToIntArrayMap.GetMutableNestedMapRef<FName> should fail with TypeMismatch"), GetMutableNestedMapResult.GetError(), EPropertyBagResult::TypeMismatch);
		}

		// Set Nested Array values using mutable helper interface
		{
			TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> GetMutableNestedArrayResult = NameToIntArrayMap.GetMutableNestedArrayRef(NestedArrayName);
			AITEST_TRUE(TEXT("NameToIntArrayMap.GetMutableNestedArrayRef<FName> should succeed."), GetMutableNestedArrayResult.HasValue());
			
			FPropertyBagArrayRef MutableNestedIntArray = GetMutableNestedArrayResult.GetValue();
			AITEST_EQUAL(TEXT("MutableNestedIntArray should be empty"), MutableNestedIntArray.Num(), 0);

			MutableNestedIntArray.AddUninitializedValues(TestArray.Num());

			for (int32 i = 0; i < TestArray.Num(); i++)
			{
				EPropertyBagResult SetValueResult = MutableNestedIntArray.SetValueInt32(i, TestArray[i]);
				AITEST_EQUAL(TEXT("MutableNestedIntArray.SetValueInt32 should succeed"), SetValueResult, EPropertyBagResult::Success);
			}

			AITEST_EQUAL(TEXT("MutableNestedIntArray.Num should match expected count"), MutableNestedIntArray.Num(), TestArray.Num());
		}

		// Get Nested Array values using const helper interface
		{
			TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> GetNestedArrayResult = NameToIntArrayMap.GetNestedArrayRef(NestedArrayName);
			AITEST_TRUE(TEXT("NameToIntArrayMap.GetNestedArrayResult<FName> should succeed."), GetNestedArrayResult.HasValue());

			const FPropertyBagArrayRef& NestedIntArray = GetNestedArrayResult.GetValue();
			AITEST_EQUAL(TEXT("NestedIntArray.Num should match expected count"), NestedIntArray.Num(), TestArray.Num());

			for (int32 i = 0; i < NestedIntArray.Num(); i++)
			{
				const TValueOrError<int32, EPropertyBagResult> GetValueResult = NestedIntArray.GetValueInt32(i);
				AITEST_TRUE(TEXT("NestedIntArray.GetValueInt32 should succeed"), GetValueResult.HasValue());
				AITEST_EQUAL(TEXT("NestedIntArray.GetValueInt32 should match expected value"), GetValueResult.GetValue(), TestArray[i]);
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_MapsNestedArray, "System.StructUtils.PropertyBag.MapsNestedArray");

struct FTest_MapsNestedSet : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName NameToEnumSetMapName(TEXT("NameToEnumSetMap"));
		static const FName NestedSetName(TEXT("NestedEnumSet"));
		static const TSet<EPropertyBagTest1> TestSet = { EPropertyBagTest1::Foo, EPropertyBagTest1::Bar, EPropertyBagTest1::Baz };

		FInstancedPropertyBag Bag;

		{
			EPropertyBagAlterationResult AlterationResult = Bag.AddMapProperty(NameToEnumSetMapName, EPropertyBagPropertyType::Name, EPropertyBagPropertyType::Enum, nullptr, StaticEnum<EPropertyBagTest1>(), EPropertyBagContainerType::Set);
			AITEST_EQUAL(TEXT("Bag.AddMapProperty should succeed"), AlterationResult, EPropertyBagAlterationResult::Success);
		}

		TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(NameToEnumSetMapName);
		AITEST_TRUE(TEXT("Bag should have map: Name->EnumSet"), GetMapResult.IsValid());

		FPropertyBagMapRef NameToEnumSetMap = GetMapResult.GetValue();
		AITEST_EQUAL(TEXT("NameToEnumSetMap should be empty"), NameToEnumSetMap.Num(), 0);

		// Verify that we can't add values directly to the map
		{
			EPropertyBagResult AddResult = NameToEnumSetMap.Add(FName("NotASet"), EPropertyBagTest1::Foo);
			AITEST_EQUAL(TEXT("NameToEnumSetMap.Add<FName,Enum> should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("NameToEnumSetMap should be empty"), NameToEnumSetMap.Num(), 0);

			AddResult = NameToEnumSetMap.Add(FName("AnEnumSet"), TSet<EPropertyBagTest1>{ EPropertyBagTest1::Foo });
			AITEST_EQUAL(TEXT("NameToEnumSetMap.Add<FName,TSet<Enum>> should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("NameToEnumSetMap should be empty"), NameToEnumSetMap.Num(), 0);
		}

		// Reserve a pair for a given key
		{
			EPropertyBagResult AddResult = NameToEnumSetMap.Add(NestedSetName);
			AITEST_EQUAL(TEXT("NameToEnumSetMap.Add<FName> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("NameToEnumSetMap should have 1 pair"), NameToEnumSetMap.Num(), 1);

			AddResult = NameToEnumSetMap.Add(NestedSetName);
			AITEST_EQUAL(TEXT("NameToEnumSetMap.Add<FName> should fail with DuplicatedValue"), AddResult, EPropertyBagResult::DuplicatedValue);
			AITEST_EQUAL(TEXT("NameToEnumSetMap should have 1 pair"), NameToEnumSetMap.Num(), 1);
		}

		// Verify that we can't get helpers for other nested container types
		{
			TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> GetMutableNestedArrayResult = NameToEnumSetMap.GetMutableNestedArrayRef(NestedSetName);
			AITEST_TRUE(TEXT("NameToEnumSetMap.GetMutableNestedArrayRef<FName>"), GetMutableNestedArrayResult.HasError());
			AITEST_EQUAL(TEXT("NameToEnumSetMap.GetMutableNestedArrayRef<FName> should fail with TypeMismatch"), GetMutableNestedArrayResult.GetError(), EPropertyBagResult::TypeMismatch);

			TValueOrError<FPropertyBagMapRef, EPropertyBagResult> GetMutableNestedMapResult = NameToEnumSetMap.GetMutableNestedMapRef(NestedSetName);
			AITEST_TRUE(TEXT("NameToEnumSetMap.GetMutableNestedMapRef<FName>"), GetMutableNestedMapResult.HasError());
			AITEST_EQUAL(TEXT("NameToEnumSetMap.GetMutableNestedMapRef<FName> should fail with TypeMismatch"), GetMutableNestedMapResult.GetError(), EPropertyBagResult::TypeMismatch);
		}

		// Add Nested Set values using mutable helper interface
		{
			TValueOrError<FPropertyBagSetRef, EPropertyBagResult> GetMutableNestedSetResult = NameToEnumSetMap.GetMutableNestedSetRef(NestedSetName);
			AITEST_TRUE(TEXT("NameToEnumSetMap.GetMutableNestedSetRef<FName> should succeed."), GetMutableNestedSetResult.HasValue());

			FPropertyBagSetRef MutableNestedEnumSet = GetMutableNestedSetResult.GetValue();
			AITEST_EQUAL(TEXT("MutableNestedEnumSet should be empty"), MutableNestedEnumSet.Num(), 0);

			for (EPropertyBagTest1 TestEnum : TestSet)
			{
				EPropertyBagResult AddValueResult = MutableNestedEnumSet.AddValueEnum(TestEnum);
				AITEST_EQUAL(TEXT("MutableNestedEnumSet.AddValueEnum should succeed"), AddValueResult, EPropertyBagResult::Success);
			}

			AITEST_EQUAL(TEXT("MutableNestedEnumSet.Num should match expected count"), MutableNestedEnumSet.Num(), TestSet.Num());
		}

		// Check Nested Set values using const helper interface
		{
			TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> GetNestedSetResult = NameToEnumSetMap.GetNestedSetRef(NestedSetName);
			AITEST_TRUE(TEXT("NameToEnumSetMap.GetNestedSetRef<FName> should succeed."), GetNestedSetResult.HasValue());
			
			const FPropertyBagSetRef& NestedEnumSet = GetNestedSetResult.GetValue();
			AITEST_EQUAL(TEXT("NestedEnumSet.Num should match expected count"), NestedEnumSet.Num(), TestSet.Num());

			for (EPropertyBagTest1 TestEnum : TestSet)
			{
				TValueOrError<bool, EPropertyBagResult> ContainsResult = NestedEnumSet.Contains(TestEnum);
				AITEST_TRUE(TEXT("NestedEnumSet.Contains should have value"), ContainsResult.HasValue());
				AITEST_TRUE(TEXT("NestedEnumSet.Contains should succeed."), ContainsResult.GetValue());
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_MapsNestedSet, "System.StructUtils.PropertyBag.MapsNestedSet");

struct FTest_MapsNestedMap : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName NameToVectorMapMapName(TEXT("NameToVectorMapMap"));
		static const FName NestedMapName(TEXT("NestedVectorMap"));
		static const TMap<FName, FVector> TestMap = { { FName("OneTwoThree"), FVector{1, 2, 3} }, { FName("FourFiveSix"), FVector{4, 5, 6} } };

		FInstancedPropertyBag Bag;

		{
			EPropertyBagAlterationResult AlterationResult = Bag.AddMapProperty(NameToVectorMapMapName, EPropertyBagPropertyType::Name, EPropertyBagPropertyType::Struct, nullptr, TBaseStructure<FVector>::Get(), EPropertyBagContainerType::Map);
			AITEST_EQUAL(TEXT("Bag.AddMapProperty should succeed"), AlterationResult, EPropertyBagAlterationResult::Success);
		}

		TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapResult = Bag.GetMapRef(NameToVectorMapMapName);
		AITEST_TRUE(TEXT("Bag should have map: Name->VectorMap"), GetMapResult.IsValid());

		FPropertyBagMapRef NameToVectorMapMap = GetMapResult.GetValue();
		AITEST_EQUAL(TEXT("NameToVectorMapMap should be empty"), NameToVectorMapMap.Num(), 0);

		// Verify that we can't add values directly to the map
		{
			EPropertyBagResult AddResult = NameToVectorMapMap.Add(FName("NotAMap"), FVector{ 1, 2, 3});
			AITEST_EQUAL(TEXT("NameToVectorMapMap.Add<FName,FVector> should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("NameToVectorMapMap should be empty"), NameToVectorMapMap.Num(), 0);

			AddResult = NameToVectorMapMap.Add(FName("AVectorMap"), TMap<FName, FVector>{ { FName("OneTwoThree"), FVector{ 1, 2, 3 } } });
			AITEST_EQUAL(TEXT("NameToVectorMapMap.Add<FName,TMap<FName, FVector>> should fail with TypeMismatch"), AddResult, EPropertyBagResult::TypeMismatch);
			AITEST_EQUAL(TEXT("NameToVectorMapMap should be empty"), NameToVectorMapMap.Num(), 0);
		}

		// Reserve a pair for a given key
		{
			EPropertyBagResult AddResult = NameToVectorMapMap.Add(NestedMapName);
			AITEST_EQUAL(TEXT("NameToVectorMapMap.Add<FName> should succeed"), AddResult, EPropertyBagResult::Success);
			AITEST_EQUAL(TEXT("NameToVectorMapMap should have 1 pair"), NameToVectorMapMap.Num(), 1);

			AddResult = NameToVectorMapMap.Add(NestedMapName);
			AITEST_EQUAL(TEXT("NameToVectorMapMap.Add<FName> should fail with DuplicatedValue"), AddResult, EPropertyBagResult::DuplicatedValue);
			AITEST_EQUAL(TEXT("NameToVectorMapMap should have 1 pair"), NameToVectorMapMap.Num(), 1);
		}

		// Verify that we can't get helpers for other nested container types
		{
			TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> GetMutableNestedArrayResult = NameToVectorMapMap.GetMutableNestedArrayRef(NestedMapName);
			AITEST_TRUE(TEXT("NameToVectorMapMap.GetMutableNestedArrayRef<FName>"), GetMutableNestedArrayResult.HasError());
			AITEST_EQUAL(TEXT("NameToVectorMapMap.GetMutableNestedArrayRef<FName> should fail with TypeMismatch"), GetMutableNestedArrayResult.GetError(), EPropertyBagResult::TypeMismatch);

			TValueOrError<FPropertyBagSetRef, EPropertyBagResult> GetMutableNestedSetResult = NameToVectorMapMap.GetMutableNestedSetRef(NestedMapName);
			AITEST_TRUE(TEXT("NameToVectorMapMap.GetMutableNestedSetRef<FName>"), GetMutableNestedSetResult.HasError());
			AITEST_EQUAL(TEXT("NameToVectorMapMap.GetMutableNestedSetRef<FName> should fail with TypeMismatch"), GetMutableNestedSetResult.GetError(), EPropertyBagResult::TypeMismatch);
		}

		// Add Nested Map values using mutable helper interface
		{
			TValueOrError<FPropertyBagMapRef, EPropertyBagResult> GetMutableNestedMapResult = NameToVectorMapMap.GetMutableNestedMapRef(NestedMapName);
			AITEST_TRUE(TEXT("NameToVectorMapMap.GetMutableNestedMapRef<FName> should succeed."), GetMutableNestedMapResult.HasValue());

			FPropertyBagMapRef MutableNestedVectorMap = GetMutableNestedMapResult.GetValue();
			AITEST_EQUAL(TEXT("MutableNestedVectorMap should be empty"), MutableNestedVectorMap.Num(), 0);

			for (const TPair<FName, FVector>& Pair : TestMap)
			{
				EPropertyBagResult AddValueResult = MutableNestedVectorMap.Add(Pair.Key, Pair.Value);
				AITEST_EQUAL(TEXT("MutableNestedVectorMap.Add should succeed"), AddValueResult, EPropertyBagResult::Success);
			}

			AITEST_EQUAL(TEXT("MutableNestedVectorMap.Num should match expected count"), MutableNestedVectorMap.Num(), TestMap.Num());
		}

		// Check Nested Map values using const helper interface
		{
			TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetNestedMapResult = NameToVectorMapMap.GetNestedMapRef(NestedMapName);
			AITEST_TRUE(TEXT("NameToVectorMapMap.GetNestedMapRef<FName> should succeed."), GetNestedMapResult.HasValue());

			const FPropertyBagMapRef& NestedVectorMap = GetNestedMapResult.GetValue();
			AITEST_EQUAL(TEXT("NestedVectorMap.Num should match expected count"), NestedVectorMap.Num(), TestMap.Num());

			for (const TPair<FName, FVector>& Pair : TestMap)
			{
				TValueOrError<bool, EPropertyBagResult> ContainsResult = NestedVectorMap.Contains(Pair.Key);
				AITEST_TRUE(TEXT("NestedVectorMap.Contains should have value"), ContainsResult.HasValue());
				AITEST_TRUE(TEXT("NestedVectorMap.Contains should succeed."), ContainsResult.GetValue());

				TValueOrError<FVector, EPropertyBagResult> FindResult = NestedVectorMap.Find<FName, FVector>(Pair.Key);
				AITEST_TRUE(TEXT("NestedVectorMap.Find should have value"), FindResult.HasValue());
				AITEST_EQUAL(TEXT("NestedVectorMap.Find should match expected value"), FindResult.GetValue(), Pair.Value);
			}
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_MapsNestedMap, "System.StructUtils.PropertyBag.MapsNestedMap");

struct FTest_SameBag : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName TemperatureName(TEXT("Temperature"));
		static const FName CountName(TEXT("Count"));
		static const FName AmountName(TEXT("Amount"));

		FInstancedPropertyBag BagA;
		BagA.AddProperties({
			{ TemperatureName, EPropertyBagPropertyType::Float },
			{ CountName, EPropertyBagPropertyType::Int32 }
		});


		FInstancedPropertyBag BagB;
		BagB.AddProperties({
			{ TemperatureName, EPropertyBagPropertyType::Float },
			{ CountName, EPropertyBagPropertyType::Int32 }
		});

		// Same descriptors should result in same bag struct
		AITEST_TRUE(TEXT("Property bags should match"), BagA.GetPropertyBagStruct() == BagB.GetPropertyBagStruct());

		//@TODO fix those cases
		//// Add a new property. They should not match.
		//BagB.AddProperties({ {AmountName, EPropertyBagPropertyType::UInt64} });
		//AITEST_FALSE(TEXT("Property bags not should match after adding a new property"), BagA.GetPropertyBagStruct() == BagB.GetPropertyBagStruct());

		//// Remove previous property. They should now matches.
		//BagB.RemovePropertiesByName({AmountName});
		//AITEST_TRUE(TEXT("Property bags should match after removing the new property"), BagA.GetPropertyBagStruct() == BagB.GetPropertyBagStruct());

		//// Remove exiting property. They should not matches.
		//BagB.RemovePropertiesByName({CountName});
		//AITEST_FALSE(TEXT("Property bags should not match after removing a property"), BagA.GetPropertyBagStruct() == BagB.GetPropertyBagStruct());

		//// Re-adding the property. They should now matches.
		//BagB.AddProperties({ {CountName, EPropertyBagPropertyType::Int32} });
		//AITEST_TRUE(TEXT("Property bags should match after readding the property"), BagA.GetPropertyBagStruct() == BagB.GetPropertyBagStruct());

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_SameBag, "System.StructUtils.PropertyBag.SameBag");

struct FTest_StructOfStructRefCounting : FAITestBase
{
	virtual bool InstantTest() override
	{
		static const FName InnerName(TEXT("Inner"));
		static const FName StringName(TEXT("String"));

		// Create a first bag with an instance, and then a second bag that has the first bag as member.
		// Use a string type, because with previous behavior, the property bag would be marked "ZeroConstructor" (since String can be zero constructed)
		// but will still have to call a destructor (because strings are not trivially destructible). 
		// It is expected that the refcount of the first bag would be 2, and the ref count of the second bag to be 1.
		// Since RefCounting of the property bag is protected, we can't check explicitly the refcount. But
		// if refcounting is not correct, releasing the instances would throw an error and will make the test fail.
		// At the end of the block, instances will be released and will go through decrementing the refcount.
		// We'll also check that the property bags are not zero constructed and have a destructor (in order for the ref count to be incremented/decremented)

		{
			FInstancedPropertyBag InnerBag;
			InnerBag.AddProperties({
				{ StringName, EPropertyBagPropertyType::String },
			});

			const UPropertyBag* InnerPropertyBag = InnerBag.GetPropertyBagStruct();
			AITEST_NOT_NULL(TEXT("Inner property bag is valid"), InnerPropertyBag);
			AITEST_TRUE(TEXT("Inner property bag is not zero constructible"), (InnerPropertyBag->StructFlags & STRUCT_ZeroConstructor) == 0);
			AITEST_TRUE(TEXT("Inner property bag has a destructor"), (InnerPropertyBag->StructFlags & STRUCT_NoDestructor) == 0);
			AITEST_TRUE(TEXT("Inner property bag is not plain old data"), (InnerPropertyBag->StructFlags & STRUCT_IsPlainOldData) == 0);

			FInstancedPropertyBag OuterBag;
			OuterBag.AddProperties({
				{ InnerName, EPropertyBagContainerType::None, EPropertyBagPropertyType::Struct, InnerPropertyBag }
			});
		
			const UPropertyBag* OuterPropertyBag = OuterBag.GetPropertyBagStruct();
			AITEST_NOT_NULL(TEXT("Outer property bag is valid"), OuterPropertyBag);
			AITEST_TRUE(TEXT("Outer property bag is not zero constructible"), (OuterPropertyBag->StructFlags & STRUCT_ZeroConstructor) == 0);
			AITEST_TRUE(TEXT("Outer property bag has a destructor"), (OuterPropertyBag->StructFlags & STRUCT_NoDestructor) == 0);
			AITEST_TRUE(TEXT("Outer property bag is not plain old data"), (OuterPropertyBag->StructFlags & STRUCT_IsPlainOldData) == 0);
		}

		return true;
	}
};
IMPLEMENT_AI_INSTANT_TEST(FTest_StructOfStructRefCounting, "System.StructUtils.PropertyBag.StructOfStructRefCounting");

} // FPropertyBagTest

#undef LOCTEXT_NAMESPACE
