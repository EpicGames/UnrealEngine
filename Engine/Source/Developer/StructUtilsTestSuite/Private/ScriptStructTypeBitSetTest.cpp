// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "StructUtils/StructTypeBitSet.h"
#include "StructUtilsTestTypes.h"

#if WITH_STRUCTUTILS_DEBUG

#define LOCTEXT_NAMESPACE "StructUtilsTests"

DECLARE_STRUCTTYPEBITSET(FTestStructBaseBitSet, FTestStructSimpleBase);
DEFINE_TYPEBITSET(FTestStructBaseBitSet);

namespace FStructTypeBitSetTests
{
struct FTestStructBaseBitSetStructTracker
{
	static FStructTracker StructTracker;
};
FStructTracker FTestStructBaseBitSetStructTracker::StructTracker([](){return FTestStructSimpleBase::StaticStruct();});

struct FTestStructBitSet : public TStructTypeBitSet<FTestStructSimple, FTestStructBaseBitSetStructTracker>
{
	using Super = TStructTypeBitSet<FTestStructSimple, FTestStructBaseBitSetStructTracker>;
	FTestStructBitSet() = default;
	FTestStructBitSet(const Super& Other)
		: Super(Other)
	{

	}

	void AddBit(const int32 Index)
	{
		DebugGetMutableStructTypesBitArray().PadToNum(Index + 1, false);
		DebugGetMutableStructTypesBitArray()[Index] = true;
	}

	void RemoveBit(const int32 Index)
	{
		DebugGetMutableStructTypesBitArray()[Index] = false;
	}

	bool TestBit(const int32 Index) const
	{
		return Index >= 0 && Index < DebugGetStructTypesBitArray().Num() && DebugGetStructTypesBitArray()[Index];
	}
};

struct FStructUtilsTest_BitSetEquivalence : FAITestBase
{
	virtual bool InstantTest() override
	{
		FTestStructBitSet CollectionA;
		FTestStructBitSet CollectionB;

		AITEST_TRUE("Empty collections are equivalent", CollectionA.IsEquivalent(CollectionB));
		AITEST_TRUE("Equivalence check is commutative", CollectionA.IsEquivalent(CollectionB) == CollectionB.IsEquivalent(CollectionA));

		CollectionA.AddBit(1);
		AITEST_FALSE("Given collections are not equivalent", CollectionA.IsEquivalent(CollectionB));
		AITEST_TRUE("Equivalence check is commutative", CollectionA.IsEquivalent(CollectionB) == CollectionB.IsEquivalent(CollectionA));

		CollectionB.AddBit(1);
		AITEST_TRUE("Given collections are equivalent", CollectionA.IsEquivalent(CollectionB));
		AITEST_TRUE("Equivalence check is commutative", CollectionA.IsEquivalent(CollectionB) == CollectionB.IsEquivalent(CollectionA));

		CollectionA.AddBit(124);
		AITEST_FALSE("Given collections are not equivalent", CollectionA.IsEquivalent(CollectionB));
		AITEST_TRUE("Equivalence check is commutative", CollectionA.IsEquivalent(CollectionB) == CollectionB.IsEquivalent(CollectionA));

		CollectionA.RemoveBit(124);
		AITEST_TRUE("Given collections are equivalent", CollectionA.IsEquivalent(CollectionB));
		AITEST_TRUE("Equivalence check is commutative", CollectionA.IsEquivalent(CollectionB) == CollectionB.IsEquivalent(CollectionA));

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetEquivalence, "System.StructUtils.BitSet.Equivalence");

struct FStructUtilsTest_BitSetEmptiness : FAITestBase
{
	virtual bool InstantTest() override
	{
		FTestStructBitSet Collection;

		AITEST_TRUE("New collection is empty", Collection.IsEmpty());
				
		Collection.AddBit(125);
		AITEST_FALSE("Extended collection is not empty", Collection.IsEmpty());
		
		Collection.RemoveBit(125);
		AITEST_TRUE("Removing the remote bit should make the collection empty again", Collection.IsEmpty());

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetEmptiness, "System.StructUtils.BitSet.Emptiness");

struct FStructUtilsTest_BitSetComparison : FAITestBase
{
	virtual bool InstantTest() override
	{
		FTestStructBitSet CollectionA;
		FTestStructBitSet CollectionNone;
		FTestStructBitSet CollectionAll;
		FTestStructBitSet CollectionSome;

		CollectionA.AddBit(1);
		CollectionA.AddBit(32);
		CollectionSome = CollectionA;
		CollectionSome.AddBit(111);

		CollectionA.AddBit(65);
		CollectionAll = CollectionA;

		CollectionA.AddBit(76);

		CollectionNone.AddBit(2);
		CollectionNone.AddBit(77);

		AITEST_TRUE("Given collection should confirm it has all its element", CollectionA.HasAll(CollectionA));
		AITEST_TRUE("CollectionA has all the elements indicated by CollectionAll set", CollectionA.HasAll(CollectionAll));
		AITEST_FALSE("HasAll is not commutative", CollectionAll.HasAll(CollectionA));

		AITEST_TRUE("CollectionA has none of the elements stored in CollectionNone", CollectionA.HasNone(CollectionNone));
		AITEST_TRUE("HasNone is commutative", CollectionNone.HasNone(CollectionA));

		AITEST_TRUE("", CollectionA.HasAny(CollectionSome));
		AITEST_TRUE("HasAny is commutative", CollectionSome.HasAny(CollectionA));

		FTestStructBitSet CollectionEmpty;
		AITEST_NOT_EQUAL("Empty bitset is not equal to non-empty", CollectionEmpty, CollectionNone);

		FTestStructBitSet CollectionEmptied = CollectionNone;
		CollectionEmptied.RemoveBit(2);
		CollectionEmptied.RemoveBit(77);
		AITEST_EQUAL("Empty bitset is equal to an emptied one", CollectionEmpty, CollectionEmptied);

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetComparison, "System.StructUtils.BitSet.Comparison");

struct FStructUtilsTest_BitSetSubtraction : FAITestBase
{
	virtual bool InstantTest() override
	{
		constexpr int TotalBits = 60;
		constexpr int BitsToClear = 40;

		FTestStructBitSet CollectionA;
		FTestStructBitSet CollectionB;
		
		for (int i = 0; i < TotalBits; ++i)
		{
			CollectionA.AddBit(i);
		}
		for (int i = 0; i < BitsToClear; ++i)
		{
			CollectionB.AddBit(i);
		}
		
		FTestStructBitSet CollectionC = CollectionA - CollectionB;

		for (int i = 0; i < BitsToClear; ++i)
		{
			AITEST_FALSE("Testing expected bit cleared", CollectionC.TestBit(i));
		}
		for (int i = BitsToClear; i < TotalBits; ++i)
		{
			AITEST_TRUE("Testing expected bit remaining", CollectionC.TestBit(i));
		}

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetSubtraction, "System.StructUtils.BitSet.Subtraction");

struct FStructUtilsTest_BitSetOverlap : FAITestBase
{
	virtual bool InstantTest() override
	{
		constexpr int BitsACount = 40;
		constexpr int OverlapBitsCount = 10;
		constexpr int BitsBCount = 30;

		FTestStructBitSet CollectionA;
		FTestStructBitSet CollectionB;

		for (int i = 0; i < BitsACount; ++i)
		{
			CollectionA.AddBit(i);
		}
		for (int i = 0; i < BitsBCount; ++i)
		{
			CollectionB.AddBit(BitsACount - OverlapBitsCount + i);
		}

		FTestStructBitSet CollectionC = CollectionA & CollectionB;
		FTestStructBitSet CollectionD = CollectionB & CollectionA;

		AITEST_TRUE("Overlap operator is commutative", CollectionC.IsEquivalent(CollectionD));

		int j = 0;
		for (int i = 0; i < BitsACount - OverlapBitsCount; ++i, ++j)
		{
			AITEST_FALSE("Testing not-overlapping bits", CollectionC.TestBit(j));
		}
		for (int i = 0; i < OverlapBitsCount; ++i, ++j)
		{
			AITEST_TRUE("Testing overlapping bits", CollectionC.TestBit(j));
		}
		for (int i = 0; i < BitsBCount - OverlapBitsCount; ++i, ++j)
		{
			AITEST_FALSE("Testing remaining non-overlapping bits", CollectionC.TestBit(j));
		}

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetOverlap, "System.StructUtils.BitSet.Overlap");

struct FStructUtilsTest_BitSetHash : FAITestBase
{
	virtual bool InstantTest() override
	{
		FTestStructBitSet EmptyCollection;
		FTestStructBitSet CollectionA;
		FTestStructBitSet CollectionB;

		CollectionA.AddBit(9);
		CollectionB.AddBit(9);
		CollectionB.AddBit(1024);

		const uint32 HashA = GetTypeHash(CollectionA);
		const uint32 HashB = GetTypeHash(CollectionB);

		AITEST_NOT_EQUAL("Two distinct bit sets should have distinct hashes", HashA, HashB);

		CollectionB.RemoveBit(1024);
		const uint32 HashB2 = GetTypeHash(CollectionB);
		AITEST_EQUAL("Two bit sets of the same composition should have have identical hashes", HashA, HashB2);
				
		CollectionB.RemoveBit(9);
		const uint32 HashEmpty = GetTypeHash(EmptyCollection);
		const uint32 HashEmptyB = GetTypeHash(CollectionB);
		AITEST_EQUAL("An emptied bit set needs to have the same hash as an empty bit set", HashEmpty, HashEmptyB);

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetHash, "System.StructUtils.BitSet.Hash");

struct FStructUtilsTest_CompileTimeBitSet : FAITestBase
{
	virtual bool InstantTest() override
	{
		FTestStructBaseBitSet::DebugResetStructTypeMappingInfo();

		FTestStructBaseBitSet BitSet1 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple1>();
		FTestStructBaseBitSet BitSet3 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple3>();
		FTestStructBaseBitSet BitSet5 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple5>();
		FTestStructBaseBitSet BitSet4 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple4>();
		FTestStructBaseBitSet BitSet2 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple2>();
		FTestStructBaseBitSet BitSet6 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple6>();
		FTestStructBaseBitSet BitSet7 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple7>();

		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet1.IsEquivalent(BitSet2));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet1.IsEquivalent(BitSet5));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet2.IsEquivalent(BitSet1));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet2.IsEquivalent(BitSet5));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet5.IsEquivalent(BitSet1));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet5.IsEquivalent(BitSet2));
		
		FTestStructBaseBitSet BitSet15;
		BitSet15.Add<FTestStructSimple1>();
		BitSet15.Add<FTestStructSimple5>();
		AITEST_TRUE("BitSets created for a list of types should be equivalent to the combination of BitSets to individual types", BitSet15.IsEquivalent(BitSet1 | BitSet5));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet2));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet3));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet4));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet6));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet7));

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_CompileTimeBitSet, "System.StructUtils.BitSet.CompileTime");

struct FStructUtilsTest_CompileTimeBitSetReversed : FAITestBase
{
	virtual bool InstantTest() override
	{
		FTestStructBaseBitSet::DebugResetStructTypeMappingInfo();

		FTestStructBaseBitSet BitSet15;
		BitSet15.Add<FTestStructSimple1>();
		BitSet15.Add<FTestStructSimple5>();

		FTestStructBaseBitSet BitSet1 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple1>();
		FTestStructBaseBitSet BitSet3 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple3>();
		FTestStructBaseBitSet BitSet5 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple5>();
		FTestStructBaseBitSet BitSet4 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple4>();
		FTestStructBaseBitSet BitSet2 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple2>();
		FTestStructBaseBitSet BitSet6 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple6>();
		FTestStructBaseBitSet BitSet7 = FTestStructBaseBitSet::GetTypeBitSet<FTestStructSimple7>();

		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet1.IsEquivalent(BitSet2));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet1.IsEquivalent(BitSet5));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet2.IsEquivalent(BitSet1));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet2.IsEquivalent(BitSet5));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet5.IsEquivalent(BitSet1));
		AITEST_FALSE("BitSets created for different types should not be equivalent", BitSet5.IsEquivalent(BitSet2));
		
		AITEST_TRUE("BitSets created for a list of types should be equivalent to the combination of BitSets to individual types", BitSet15.IsEquivalent(BitSet1 | BitSet5));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet2));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet3));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet4));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet6));
		AITEST_FALSE("BitSets created for a list of types should not contain bit for other types", BitSet15.HasAny(BitSet7));

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_CompileTimeBitSetReversed, "System.StructUtils.BitSet.CompileTime_Reversed");

struct FStructUtilsTest_BitSetIndexIteratorBase : FAITestBase
{
	TArray<int32> Indices;
	FTestStructBitSet BitSet;
	bool bValue = true;

	FStructUtilsTest_BitSetIndexIteratorBase()
		: Indices({ 0, 3, 9, 230 })
	{}

	virtual bool SetUp() override
	{
		for (int32 Index : Indices)
		{
			BitSet.AddBit(Index);
		}
		return FAITestBase::SetUp();
	}

	virtual bool InstantTest() override
	{
		int32 i = 0;
		for (auto It = BitSet.GetIndexIterator(/*bValue=*/bValue); It; ++It, ++i)
		{
			AITEST_EQUAL("Indices fetched by FIndexIterator need to reflect Bitset\'s contents", Indices[i], *It);
		}
		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetIndexIteratorBase, "System.StructUtils.BitSet.IndexIterator");

struct FStructUtilsTest_BitSetIndexIteratorEmpty : FStructUtilsTest_BitSetIndexIteratorBase
{
	FStructUtilsTest_BitSetIndexIteratorEmpty()
	{
		Indices = {};
	}
};
IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetIndexIteratorEmpty, "System.StructUtils.BitSet.IndexIterator.Empty");

struct FStructUtilsTest_BitSetIndexIteratorTrailingFalse : FStructUtilsTest_BitSetIndexIteratorBase
{
	virtual bool SetUp() override
	{
		Indices.Sort();
		for (int32 Index : Indices)
		{
			BitSet.AddBit(Index);
		}
		const int32 TrailingFalseBitIndex = Indices.Last() + 1;
		BitSet.AddBit(TrailingFalseBitIndex);
		BitSet.RemoveBit(TrailingFalseBitIndex);

		return FAITestBase::SetUp();
	}
};
IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetIndexIteratorTrailingFalse, "System.StructUtils.BitSet.IndexIterator.TrailingFalse");

struct FStructUtilsTest_BitSetIndexIteratorBeginningFalse : FStructUtilsTest_BitSetIndexIteratorBase
{
	virtual bool SetUp() override
	{
		Indices.Sort();
		for (int32 Index : Indices)
		{
			BitSet.AddBit(Index);
		}
		BitSet.RemoveBit(Indices[0]);
		Indices.RemoveAt(0);

		return FAITestBase::SetUp();
	}
};
IMPLEMENT_AI_INSTANT_TEST(FStructUtilsTest_BitSetIndexIteratorBeginningFalse, "System.StructUtils.BitSet.IndexIterator.BeginningFalse");

} // namespace FStructTypeBitSetTests

#undef LOCTEXT_NAMESPACE

#endif // WITH_STRUCTUTILS_DEBUG
