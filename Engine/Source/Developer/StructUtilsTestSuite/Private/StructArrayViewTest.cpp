// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "StructUtils/StructArrayView.h"
#include "StructUtilsTestTypes.h"

#define LOCTEXT_NAMESPACE "StructUtilsTests"

namespace FStructArrayViewTest
{

struct FTest_ArrayViewChangeElements : FAITestBase
{
	virtual bool InstantTest() override
	{
		constexpr float Increment = 10.f;
		TArray<float> OriginalValues = { 1,2,3,4,5,6,7,8,9,10 };
		TArray<FTestStructSimple> ValuesF;
		for (float Value : OriginalValues)
		{
			ValuesF.Add(Value);
		}
		FStructArrayView ViewF(ValuesF);

		for (int32 i = 0; i < ValuesF.Num(); ++i)
		{
			ViewF.GetAt<FTestStructSimple>(i).Float += 10.f;
		}

		{
			int32 i = 0;
			for (FStructView StructViewTest : ViewF)
			{
				AITEST_EQUAL(TEXT("FStructView should have the same value as TArray"), ValuesF[i].Float, StructViewTest.Get<FTestStructSimple>().Float);
				AITEST_EQUAL(TEXT("FStructArrayView GetAt() should have the same value as TArray"), ValuesF[i].Float, ViewF.GetAt< FTestStructSimple>(i).Float);
				AITEST_TRUE(TEXT("FStructArrayView GetPtrAt() should be non Null"), ViewF.GetPtrAt< FTestStructSimple>(i) != nullptr);
				AITEST_EQUAL(TEXT("FStructArrayView GetPtrAt() should have the same value as TArray"), ValuesF[i].Float, ViewF.GetPtrAt< FTestStructSimple>(i)->Float);
				AITEST_EQUAL(TEXT("FStructArrayView operator[] should have the same value as TArray"), ValuesF[i].Float, ViewF[i].Get<FTestStructSimple>().Float);
				AITEST_EQUAL(TEXT("FStructArrayView Element should reflect the change done to it"), ViewF.GetAt<FTestStructSimple>(i).Float, OriginalValues[i] + Increment);
				AITEST_EQUAL(TEXT("FStructArrayView Element should point at the same adress in memory as TArray"), (void*)&ValuesF[i], ViewF.GetDataAt(i));

				++i;
			}
			AITEST_EQUAL(TEXT("FStructArrayView should have the same number of elements value as TArray"), i, ValuesF.Num());
		}
		
		{
			FStructArrayView ViewLeft(ViewF);
			const int32 LeftNum = 3;
			ViewLeft.LeftInline(LeftNum);
			AITEST_EQUAL(TEXT("Incorrect LeftInline() modified FStructArrayView Num"), LeftNum, ViewLeft.Num());

			int32 i = 0;
			for (FStructView StructViewTest : ViewLeft)
			{
				AITEST_EQUAL(TEXT("LeftInline() modified FStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FStructArrayView ViewLeftChop(ViewF);
			const int32 LeftChopNum = 4;
			ViewLeftChop.LeftChopInline(LeftChopNum);
			AITEST_EQUAL(TEXT("Incorrect LeftChopInline() modified FStructArrayView Num"), ViewF.Num() - LeftChopNum, ViewLeftChop.Num());

			int32 i = 0;
			for (FStructView StructViewTest : ViewLeftChop)
			{
				AITEST_EQUAL(TEXT("LeftChopInline() modified FStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FStructArrayView ViewRight(ViewF);
			const int32 RightNum = 2;
			ViewRight.RightInline(RightNum);
			AITEST_EQUAL(TEXT("Incorrect RightInline() modified FStructArrayView Num"), RightNum, ViewRight.Num());

			int32 i = ViewF.Num() - RightNum;
			for (FStructView StructViewTest : ViewRight)
			{
				AITEST_EQUAL(TEXT("RightInline() modified FStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FStructArrayView ViewRightChop(ViewF);
			const int32 RightChopNum = 6;
			ViewRightChop.RightChopInline(RightChopNum);
			AITEST_EQUAL(TEXT("Incorrect RightChopInline() modified FStructArrayView Num"), ViewF.Num() - RightChopNum, ViewRightChop.Num());

			int32 i = RightChopNum;
			for (FStructView StructViewTest : ViewRightChop)
			{
				AITEST_EQUAL(TEXT("RightChopInline() modified FStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FStructArrayView ViewMid(ViewF);
			const int32 MidPositon = 3;
			const int32 MidNum = 4;
			ViewMid.MidInline(MidPositon, MidNum);
			AITEST_EQUAL(TEXT("Incorrect MidInline() modified FStructArrayView Num"), MidNum, ViewMid.Num());

			int32 i = MidPositon;
			for (FStructView StructViewTest : ViewMid)
			{
				AITEST_EQUAL(TEXT("MidInline() modified FStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FStructArrayView ViewSlice(ViewF);
			const int32 SlicePositon = 2;
			const int32 SliceNum = 5;
			ViewSlice = ViewSlice.Slice(SlicePositon, SliceNum);
			AITEST_EQUAL(TEXT("Incorrect Slice() modified FStructArrayView Num"), SliceNum, ViewSlice.Num());

			int32 i = SlicePositon;
			for (FStructView StructViewTest : ViewSlice)
			{
				AITEST_EQUAL(TEXT("Slice() modified FStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FTest_ArrayViewChangeElements, "System.StructUtils.StructArrayView.ChangeElements");

struct FTest_ConstArrayViewChangeElements : FAITestBase
{
	virtual bool InstantTest() override
	{
		constexpr float Increment = 10.f;
		TArray<float> OriginalValues = { 1,2,3,4,5,6,7,8,9,10 };
		TArray<FTestStructSimple> ValuesF;
		for (float Value : OriginalValues)
		{
			ValuesF.Add(Value);
		}
		FStructArrayView ViewF(ValuesF);
		FConstStructArrayView ViewConstF(ViewF);

		for (int32 i = 0; i < ValuesF.Num(); ++i)
		{
			ViewF.GetAt<FTestStructSimple>(i).Float += 10.f;
		}

		{
			int32 i = 0;
			for (FConstStructView StructViewTest : ViewConstF)
			{
				AITEST_EQUAL(TEXT("FConstStructView should have the same value as TArray"), ValuesF[i].Float, StructViewTest.Get<const FTestStructSimple>().Float);
				AITEST_EQUAL(TEXT("FConstStructArrayView GetAt() should have the same value as TArray"), ValuesF[i].Float, ViewConstF.GetAt<const FTestStructSimple>(i).Float);
				AITEST_TRUE(TEXT("FConstStructArrayView GetPtrAt() should be non Null"), ViewConstF.GetPtrAt<const FTestStructSimple>(i) != nullptr);
				AITEST_EQUAL(TEXT("FConstStructArrayView GetPtrAt() should have the same value as TArray"), ValuesF[i].Float, ViewConstF.GetPtrAt<const FTestStructSimple>(i)->Float);
				AITEST_EQUAL(TEXT("FConstStructArrayView operator[] should have the same value as TArray"), ValuesF[i].Float, ViewConstF[i].Get<const FTestStructSimple>().Float);
				AITEST_EQUAL(TEXT("FConstStructArrayView Element should reflect the change done to it"), ViewConstF.GetAt<const FTestStructSimple>(i).Float, OriginalValues[i] + Increment);
				AITEST_EQUAL(TEXT("FConstStructArrayView Element should point at the same adress in memory as TArray"), (void*)&ValuesF[i], ViewConstF.GetDataAt(i));

				++i;
			}
			AITEST_EQUAL(TEXT("FConstStructArrayView should have the same number of elements value as TArray"), i, ViewConstF.Num());
		}

		{
			FConstStructArrayView ViewLeft(ViewF);
			const int32 LeftNum = 3;
			ViewLeft.LeftInline(LeftNum);
			AITEST_EQUAL(TEXT("Incorrect LeftInline() modified FConstStructArrayView Num"), LeftNum, ViewLeft.Num());

			int32 i = 0;
			for (FConstStructView StructViewTest : ViewLeft)
			{
				AITEST_EQUAL(TEXT("LeftInline() modified FConstStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<const FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FConstStructArrayView ViewLeftChop(ViewF);
			const int32 LeftChopNum = 4;
			ViewLeftChop.LeftChopInline(LeftChopNum);
			AITEST_EQUAL(TEXT("Incorrect LeftChopInline() modified FConstStructArrayView Num"), ViewF.Num() - LeftChopNum, ViewLeftChop.Num());

			int32 i = 0;
			for (FConstStructView StructViewTest : ViewLeftChop)
			{
				AITEST_EQUAL(TEXT("LeftChopInline() modified FConstStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<const FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FConstStructArrayView ViewRight(ViewF);
			const int32 RightNum = 2;
			ViewRight.RightInline(RightNum);
			AITEST_EQUAL(TEXT("Incorrect RightInline() modified FConstStructArrayView Num"), RightNum, ViewRight.Num());

			int32 i = ViewF.Num() - RightNum;
			for (FConstStructView StructViewTest : ViewRight)
			{
				AITEST_EQUAL(TEXT("RightInline() modified FConstStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<const FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FConstStructArrayView ViewRightChop(ViewF);
			const int32 RightChopNum = 6;
			ViewRightChop.RightChopInline(RightChopNum);
			AITEST_EQUAL(TEXT("Incorrect RightChopInline() modified FConstStructArrayView Num"), ViewF.Num() - RightChopNum, ViewRightChop.Num());

			int32 i = RightChopNum;
			for (FConstStructView StructViewTest : ViewRightChop)
			{
				AITEST_EQUAL(TEXT("RightChopInline() modified FConstStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<const FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FConstStructArrayView ViewMid(ViewF);
			const int32 MidPositon = 3;
			const int32 MidNum = 4;
			ViewMid.MidInline(MidPositon, MidNum);
			AITEST_EQUAL(TEXT("Incorrect MidInline() modified FConstStructArrayView Num"), MidNum, ViewMid.Num());

			int32 i = MidPositon;
			for (FConstStructView StructViewTest : ViewMid)
			{
				AITEST_EQUAL(TEXT("MidInline() modified FConstStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<const FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		{
			FConstStructArrayView ViewSlice(ViewF);
			const int32 SlicePositon = 2;
			const int32 SliceNum = 5;
			ViewSlice = ViewSlice.Slice(SlicePositon, SliceNum);
			AITEST_EQUAL(TEXT("Incorrect Slice() modified FConstStructArrayView Num"), SliceNum, ViewSlice.Num());

			int32 i = SlicePositon;
			for (FConstStructView StructViewTest : ViewSlice)
			{
				AITEST_EQUAL(TEXT("Slice() modified FConstStructArrayView element should be the same as correpsonding element in TArray"), StructViewTest.Get<const FTestStructSimple>().Float, ValuesF[i].Float);

				++i;
			}
		}

		return true;
	}
};

IMPLEMENT_AI_INSTANT_TEST(FTest_ConstArrayViewChangeElements, "System.StructUtils.StructArrayView.ChangeElementsConst");
} // FStructArrayViewTest

#undef LOCTEXT_NAMESPACE
