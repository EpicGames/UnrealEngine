// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "StructUtils/SharedStruct.h"
#include "StructUtilsTestTypes.h"

#define LOCTEXT_NAMESPACE "StructUtilsTests"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FStructSharedStructTest
{
	struct FTest_SharedStructCreate : FAITestBase
	{
		virtual bool InstantTest() override
		{
			{
				FSharedStruct SharedStruct = FSharedStruct::Make<FTestStructSimpleNonZeroDefault>();

				AITEST_EQUAL("FSharedStruct default initialized from Make should have same value as default constructed", FTestStructSimpleNonZeroDefault(), SharedStruct.Get<FTestStructSimpleNonZeroDefault>());
			}

			{
				const float Val = 99.f;
				FTestStructSimple Simple(Val);

				FSharedStruct SharedStruct = FSharedStruct::Make(Simple);

				AITEST_EQUAL("FSharedStruct initialized from Make should have value of FTestStructSimple its initiliazed from", Val, SharedStruct.Get<FTestStructSimple>().Float);
			}

			{
				const float Val = 99.f;
				FTestStructSimple Simple(Val);

				FSharedStruct SharedStruct = FSharedStruct::Make(FTestStructSimple::StaticStruct(), (uint8*)&Simple);

				AITEST_EQUAL("FSharedStruct initialized from Make should have value of StaticStruct and memory its initiliazed from", Val, SharedStruct.Get<FTestStructSimple>().Float);
			}

			{
				const float Val = 99.f;
				FTestStructSimple Simple(Val);
				FStructView StructView = FStructView::Make(Simple);
				FSharedStruct SharedStruct = FSharedStruct::Make(StructView);

				AITEST_EQUAL("FSharedStruct initialized from Make should have value of FStructView its initiliazed from", Val, SharedStruct.Get<const FTestStructSimple>().Float);
			}

			{
				const float Val = 99.f;
				FTestStructSimple Simple(Val);

				FSharedStruct SharedStruct = FSharedStruct::Make<FTestStructSimple>(99);

				AITEST_EQUAL("FSharedStruct initialized from Make should have value reflecting TArgs", Val, SharedStruct.Get<FTestStructSimple>().Float);
			}

			return true;
		}
	};

	IMPLEMENT_AI_INSTANT_TEST(FTest_SharedStructCreate, "System.StructUtils.SharedStruct.Make");

	struct FTest_SharedStructBasic : FAITestBase
	{
		virtual bool InstantTest() override
		{
			{
				FSharedStruct SharedStruct = FSharedStruct::Make<FTestStructSimple>();
				FSharedStruct SharedStruct2(SharedStruct);

				AITEST_EQUAL("FSharedStruct and FSharedStruct should be equal from copy construction", SharedStruct, SharedStruct2);
			}

			{
				FSharedStruct SharedStruct = FSharedStruct::Make<FTestStructSimple>();
				FSharedStruct SharedStruct2;

				SharedStruct2 = SharedStruct;

				AITEST_EQUAL("FSharedStruct and FSharedStruct should be equal from copy asignment", SharedStruct, SharedStruct2);
			}

			{
				const float Val = 99.f;

				FSharedStruct SharedStruct = FSharedStruct::Make<FTestStructSimple>();
				FSharedStruct SharedStruct2(SharedStruct);

				SharedStruct.Get<FTestStructSimple>().Float = Val;

				AITEST_EQUAL("SharedStruct and SharedStruct2 should reflect the same changes to shared FTestStructSimple", SharedStruct.Get<FTestStructSimple>().Float, SharedStruct2.Get<FTestStructSimple>().Float);
			}

			{
				FSharedStruct SharedStruct;
				AITEST_FALSE("Default constructed FSharedStruct should IsValid() == false", SharedStruct.IsValid());
			}

			{
				FSharedStruct SharedStruct = FSharedStruct::Make<FTestStructSimple>();
				AITEST_TRUE("FSharedStruct created to a specific struct type should be IsValid()", SharedStruct.IsValid());
			}

			{
				FSharedStruct SharedStruct1 = FSharedStruct::Make<FTestStructSimple>();
				FSharedStruct SharedStruct2 = FSharedStruct::Make<FTestStructSimple>();
				const bool bEqual = SharedStruct1 == SharedStruct2;
				AITEST_FALSE("FSharedStruct operator== should compare what is shared and not what it contains", bEqual);
			}

			{
				FSharedStruct SharedStruct1 = FSharedStruct::Make<FTestStructSimple>();
				FSharedStruct SharedStruct2 = FSharedStruct::Make<FTestStructSimple>();
				TArray<FSharedStruct> Array;
				Array.Add(SharedStruct1);

				AITEST_TRUE("FSharedStruct should support TArray", Array.Contains(SharedStruct1));
				AITEST_FALSE("FSharedStruct should support TArray", Array.Contains(SharedStruct2));
			}

			return true;
		}
	};

	IMPLEMENT_AI_INSTANT_TEST(FTest_SharedStructBasic, "System.StructUtils.SharedStruct.Basic");

	struct FTest_ConstSharedStructCreate : FAITestBase
	{
		virtual bool InstantTest() override
		{
			{
				FConstSharedStruct SharedStruct = FConstSharedStruct::Make<FTestStructSimpleNonZeroDefault>();

				AITEST_EQUAL("FConstSharedStruct default initialized from Make should have same value as default constructed", FTestStructSimpleNonZeroDefault(), SharedStruct.Get<const FTestStructSimpleNonZeroDefault>());
			}

			{
				const float Val = 99.f;
				FTestStructSimple Simple(Val);

				FConstSharedStruct SharedStruct = FConstSharedStruct::Make(Simple);

				AITEST_EQUAL("FConstSharedStruct initialized from Make should have value of FTestStructSimple its initiliazed from", Val, SharedStruct.Get<const FTestStructSimple>().Float);
			}

			{
				const float Val = 99.f;
				FTestStructSimple Simple(Val);

				FConstSharedStruct ConstSharedStruct = FConstSharedStruct::Make(FTestStructSimple::StaticStruct(), (uint8*)&Simple);

				AITEST_EQUAL("FConstSharedStruct initialized from Make should have value of StaticStruct and memory its initiliazed from", Val, ConstSharedStruct.Get<const FTestStructSimple>().Float);
			}

			{
				const float Val = 99.f;
				FTestStructSimple Simple(Val);
				FStructView StructView = FStructView::Make(Simple);
				FConstSharedStruct ConstSharedStruct = FConstSharedStruct::Make(StructView);

				AITEST_EQUAL("FConstSharedStruct initialized from Make should have value of FStructView its initiliazed from", Val, ConstSharedStruct.Get<const FTestStructSimple>().Float);
			}

			{
				const float Val = 99.f;
				FTestStructSimple Simple(Val);

				FConstSharedStruct ConstSharedStruct = FConstSharedStruct::Make<FTestStructSimple>(99);

				AITEST_EQUAL("FConstSharedStruct initialized from Make should have value reflecting TArgs", Val, ConstSharedStruct.Get<const FTestStructSimple>().Float);
			}

			return true;
		}
	};

	IMPLEMENT_AI_INSTANT_TEST(FTest_ConstSharedStructCreate, "System.StructUtils.SharedStruct.Const.Make");

	struct FTest_ConstSharedStructBasic : FAITestBase
	{
		virtual bool InstantTest() override
		{
			{
				FConstSharedStruct ConstSharedStruct = FConstSharedStruct::Make<FTestStructSimple>();
				FConstSharedStruct ConstSharedStruct2(ConstSharedStruct);

				AITEST_EQUAL("FConstSharedStruct and FConstSharedStruct should be equal from copy construction", ConstSharedStruct, ConstSharedStruct2);
			}

			{
				FConstSharedStruct ConstSharedStruct = FConstSharedStruct::Make<FTestStructSimple>();
				FConstSharedStruct ConstSharedStruct2;
				
				ConstSharedStruct2 = ConstSharedStruct;

				AITEST_EQUAL("FConstSharedStruct and FConstSharedStruct should be equal from copy asignment", ConstSharedStruct, ConstSharedStruct2);
			}

			{
				FSharedStruct SharedStruct = FSharedStruct::Make<FTestStructSimple>();
				FConstSharedStruct ConstSharedStruct(SharedStruct);

				AITEST_TRUE("FSharedStruct and FConstSharedStruct should be equal from copy construction", SharedStruct == ConstSharedStruct);
			}

			{
				FSharedStruct SharedStruct = FSharedStruct::Make<FTestStructSimple>();
				FConstSharedStruct ConstSharedStruct;
				
				ConstSharedStruct = SharedStruct;

				AITEST_TRUE("FSharedStruct and FConstSharedStruct should be equal from copy asignment", SharedStruct == ConstSharedStruct);
			}

			{
				const float Val = 99.f;

				FSharedStruct SharedStruct = FSharedStruct::Make<FTestStructSimple>();
				FConstSharedStruct ConstSharedStruct(SharedStruct);

				SharedStruct.Get<FTestStructSimple>().Float = Val;

				AITEST_EQUAL("FSharedStruct and FConstSharedStruct should reflect the same changes to shared FTestStructSimple", SharedStruct.Get<FTestStructSimple>().Float, ConstSharedStruct.Get<const FTestStructSimple>().Float);
			}

			{
				FConstSharedStruct ConstSharedStruct;
				AITEST_FALSE("Default constructed FConstSharedStruct should IsValid() == false", ConstSharedStruct.IsValid());
			}

			{
				FConstSharedStruct ConstSharedStruct = FConstSharedStruct::Make<FTestStructSimple>();
				AITEST_TRUE("FConstSharedStruct created to a specific struct type should be IsValid()", ConstSharedStruct.IsValid());
			}

			{
				FConstSharedStruct ConstSharedStruct1 = FConstSharedStruct::Make<FTestStructSimple>();
				FConstSharedStruct ConstSharedStruct2 = FConstSharedStruct::Make<FTestStructSimple>();
				const bool bEqual = ConstSharedStruct1 == ConstSharedStruct2;
				AITEST_FALSE("FConstSharedStruct operator== should compare what is shared and not what it contains", bEqual);
			}

			{
				FConstSharedStruct ConstSharedStruct1 = FConstSharedStruct::Make<FTestStructSimple>();
				FConstSharedStruct ConstSharedStruct2 = FConstSharedStruct::Make<FTestStructSimple>();
				TArray<FConstSharedStruct> Array;
				Array.Add(ConstSharedStruct1);

				AITEST_TRUE("FConstSharedStruct should support TArray", Array.Contains(ConstSharedStruct1));
				AITEST_FALSE("FConstSharedStruct should support TArray", Array.Contains(ConstSharedStruct2));
			}

			return true;
		}
	};

	IMPLEMENT_AI_INSTANT_TEST(FTest_ConstSharedStructBasic, "System.StructUtils.SharedStruct.Const.Basic");

	struct FTest_TSharedStructBasic : FAITestBase
	{
		virtual bool InstantTest() override
		{
			{
				TSharedStruct<FTestStructSimple> SharedStruct = TSharedStruct<FTestStructSimple>::Make();
				TSharedStruct<FTestStructSimple> SharedStruct2(SharedStruct);

				AITEST_EQUAL("TSharedStruct and TSharedStruct should be equal from copy construction", SharedStruct, SharedStruct2);
			}

			{
				TSharedStruct<FTestStructSimple> SharedStruct = TSharedStruct<FTestStructSimple>::Make();
				TSharedStruct<FTestStructSimple> SharedStruct2;
				
				SharedStruct2 = SharedStruct;

				AITEST_EQUAL("TSharedStruct and TSharedStruct should be equal from copy asignment", SharedStruct, SharedStruct2);
			}
			{
				constexpr float Val = 99.f;

				TSharedStruct<FTestStructSimple> SharedStruct1 = TSharedStruct<FTestStructSimple>::Make();
				TSharedStruct<FTestStructSimple> SharedStruct2(SharedStruct1);

				SharedStruct1.Get<FTestStructSimple>().Float = Val;
				AITEST_EQUAL("FSharedStruct and TSharedStruct should reflect the same changes to shared FTestStructSimple", SharedStruct1.Get().Float, SharedStruct2.Get().Float);
			}

			{
				TSharedStruct<FTestStructSimple> SharedStructTemplated;
				AITEST_FALSE("Default constructed TSharedStruct should IsValid() == false", SharedStructTemplated.IsValid());
			}

			{
				TSharedStruct<FTestStructSimple> SharedStructTemplated = TSharedStruct<FTestStructSimple>::Make();
				AITEST_TRUE("TSharedStruct created to a specific struct type should be IsValid()", SharedStructTemplated.IsValid());
			}

			{
				TSharedStruct<FTestStructSimple> SharedStructTemplated1 = TSharedStruct<FTestStructSimple>::Make();
				TSharedStruct<FTestStructSimple> SharedStructTemplated2 = TSharedStruct<FTestStructSimple>::Make();
				const bool bEqual = SharedStructTemplated1 == SharedStructTemplated2;
				AITEST_FALSE("TSharedStruct operator== should compare what is shared and not what it contains", bEqual);
			}

			{
				TSharedStruct<FTestStructSimple> SharedStructTemplated1 = TSharedStruct<FTestStructSimple>::Make();
				TSharedStruct<FTestStructSimple> SharedStructTemplated2 = TSharedStruct<FTestStructSimple>::Make();
				TArray<TSharedStruct<FTestStructSimple>> Array;
				Array.Add(SharedStructTemplated1);

				AITEST_TRUE("TSharedStruct should support TArray", Array.Contains(SharedStructTemplated1));
				AITEST_FALSE("TSharedStruct should support TArray", Array.Contains(SharedStructTemplated2));
			}

			return true;
		}
	};

	IMPLEMENT_AI_INSTANT_TEST(FTest_TSharedStructBasic, "System.StructUtils.SharedStruct.Template.Basic");

	struct FTest_TConstSharedStructBasic : FAITestBase
	{
		virtual bool InstantTest() override
		{
			{
				TConstSharedStruct<FTestStructSimple> SharedStruct1 = TConstSharedStruct<FTestStructSimple>::Make();
				TConstSharedStruct<FTestStructSimple> SharedStruct2(SharedStruct1);

				AITEST_EQUAL("TConstSharedStruct and TConstSharedStruct should be equal from copy construction", SharedStruct1, SharedStruct2);
			}

			{
				TSharedStruct<FTestStructSimple> SharedStruct1 = TSharedStruct<FTestStructSimple>::Make();
				TConstSharedStruct<FTestStructSimple> SharedStruct2(SharedStruct1);

				const bool bEqual = SharedStruct1 == SharedStruct2;
				const bool bNotEqual = SharedStruct1 != SharedStruct2;
				AITEST_TRUE("TConstSharedStruct and TSharedStruct should be equal from copy construction", bEqual);
				AITEST_FALSE("TConstSharedStruct and TSharedStruct should be equal from copy construction", bNotEqual);
			}

			{
				TConstSharedStruct<FTestStructSimple> SharedStruct1 = TConstSharedStruct<FTestStructSimple>::Make();
				TConstSharedStruct<FTestStructSimple> SharedStruct2;
				
				SharedStruct2 = SharedStruct1;

				AITEST_EQUAL("TConstSharedStruct and TConstSharedStruct should be equal from copy asignment", SharedStruct1, SharedStruct2);
			}

			{
				TSharedStruct<FTestStructSimple> SharedStruct1 = TSharedStruct<FTestStructSimple>::Make();
				TConstSharedStruct<FTestStructSimple> SharedStruct2;
				
				SharedStruct2 = SharedStruct1;
				const bool bEqual = SharedStruct1 == SharedStruct2;
				const bool bNotEqual = SharedStruct1 != SharedStruct2;
				AITEST_TRUE("TSharedStruct and TConstSharedStruct should be equal from copy asignment", bEqual);
				AITEST_FALSE("TSharedStruct and TConstSharedStruct should be equal from copy asignment", bNotEqual);
			}

			{
				constexpr float Val = 99.f;

				TConstSharedStruct<FTestStructSimple> SharedStruct1 = TConstSharedStruct<FTestStructSimple>::Make(Val);
				TConstSharedStruct<FTestStructSimple> SharedStruct2(SharedStruct1);

				AITEST_EQUAL("TConstSharedStruct and TConstSharedStruct should reflect the same changes to shared FTestStructSimple", SharedStruct1.Get().Float, SharedStruct2.Get().Float);
				AITEST_EQUAL("TConstSharedStruct and TConstSharedStruct should reflect the same changes to shared FTestStructSimple", SharedStruct1.GetPtr()->Float, SharedStruct2.GetPtr()->Float);
				AITEST_EQUAL("TConstSharedStruct and TConstSharedStruct should reflect the same changes to shared FTestStructSimple", SharedStruct1.Get<FTestStructSimple>().Float, SharedStruct2.Get<const FTestStructSimple>().Float);
				AITEST_EQUAL("TConstSharedStruct and TConstSharedStruct should reflect the same changes to shared FTestStructSimple", SharedStruct1.GetPtr<FTestStructSimple>()->Float, SharedStruct2.GetPtr<const FTestStructSimple>()->Float);
			}

			{
				TConstSharedStruct<FTestStructSimple> SharedStructTemplated;
				AITEST_FALSE("Default constructed TConstSharedStruct should IsValid() == false", SharedStructTemplated.IsValid());
			}

			{
				TConstSharedStruct<FTestStructSimple> SharedStructTemplated = TConstSharedStruct<FTestStructSimple>::Make();
				AITEST_TRUE("TConstSharedStruct created to a specific struct type should be IsValid()", SharedStructTemplated.IsValid());
			}

			{
				TConstSharedStruct<FTestStructSimple> SharedStructTemplated1 = TConstSharedStruct<FTestStructSimple>::Make();
				TConstSharedStruct<FTestStructSimple> SharedStructTemplated2 = TConstSharedStruct<FTestStructSimple>::Make();
				const bool bEqual = SharedStructTemplated1 == SharedStructTemplated2;
				AITEST_FALSE("TConstSharedStruct operator== should compare what is shared and not what it contains", bEqual);
			}

			{
				TSharedStruct<FTestStructSimple> SharedStructTemplated1 = TSharedStruct<FTestStructSimple>::Make();
				TConstSharedStruct<FTestStructSimple> SharedStructTemplated2 = TConstSharedStruct<FTestStructSimple>::Make();
				const bool bEqual = SharedStructTemplated1 == SharedStructTemplated2;
				AITEST_FALSE("TConstSharedStruct operator== should compare what is shared and not what it contains", bEqual);
			}

			{
				TConstSharedStruct<FTestStructSimple> SharedStructTemplated1 = TConstSharedStruct<FTestStructSimple>::Make();
				TSharedStruct<FTestStructSimple> SharedStructTemplated2 = TSharedStruct<FTestStructSimple>::Make();
				const bool bEqual = SharedStructTemplated1 == SharedStructTemplated2;
				AITEST_FALSE("TConstSharedStruct operator== should compare what is shared and not what it contains", bEqual);
			}

			{
				TConstSharedStruct<FTestStructSimple> SharedStructTemplated1 = TConstSharedStruct<FTestStructSimple>::Make();
				TConstSharedStruct<FTestStructSimple> SharedStructTemplated2 = TConstSharedStruct<FTestStructSimple>::Make();
				TArray<TConstSharedStruct<FTestStructSimple>> Array;
				Array.Add(SharedStructTemplated1);

				AITEST_TRUE("TSharedStruct should support TArray", Array.Contains(SharedStructTemplated1));
				AITEST_FALSE("TSharedStruct should support TArray", Array.Contains(SharedStructTemplated2));
			}

			return true;
		}
	};

	IMPLEMENT_AI_INSTANT_TEST(FTest_TConstSharedStructBasic, "System.StructUtils.SharedStruct.Template.Const.Basic");

	struct FTest_SharedStructDestruction : FAITestBase
	{
		virtual bool InstantTest() override
		{
			int32 NumConstructedStructBefore = FTestStructNonTrivialDestructor::NumConstructed;
			{
				FTestStructNonTrivialDestructor Struct;	
				AITEST_EQUAL("Number of constructed FTestStructNonTrivialDestructor should have increased by 1 after local struct ctor", FTestStructNonTrivialDestructor::NumConstructed, NumConstructedStructBefore+1);
			}
			AITEST_EQUAL("Number of constructed FTestStructNonTrivialDestructor should be decreased by 1 after local struct dtor", FTestStructNonTrivialDestructor::NumConstructed, NumConstructedStructBefore);

			NumConstructedStructBefore = FTestStructNonTrivialDestructor::NumConstructed;
			{
				FTestStructNonTrivialDestructor Struct;
				FSharedStruct SharedStruct = FSharedStruct::Make(Struct);
				AITEST_EQUAL("Number of constructed FTestStructNonTrivialDestructor should have increased by 2 after local struct and shared struct ctors", FTestStructNonTrivialDestructor::NumConstructed, NumConstructedStructBefore+2);
			}
			AITEST_EQUAL("Number of constructed FTestStructNonTrivialDestructor should be decreased by 2 after local struct and shared struct dtors", FTestStructNonTrivialDestructor::NumConstructed, NumConstructedStructBefore);

			return true;
		}
	};

	IMPLEMENT_AI_INSTANT_TEST(FTest_SharedStructDestruction, "System.StructUtils.SharedStruct.Destruction");
}


UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
