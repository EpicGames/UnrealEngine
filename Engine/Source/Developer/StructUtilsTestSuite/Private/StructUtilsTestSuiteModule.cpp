// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtilsTestSuiteModule.h"

#define LOCTEXT_NAMESPACE "StructUtilsTestSuite"

class FStructUtilsTestSuiteModule : public IStructUtilsTestSuiteModule
{
};

IMPLEMENT_MODULE(FStructUtilsTestSuiteModule, StructUtilsTestSuite)

#undef LOCTEXT_NAMESPACE
