// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtilsTestTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StructUtilsTestTypes)

uint32 GetTypeHash(const FTestStructHashable1& other)
{
	return FCrc::MemCrc32(&other, sizeof(FTestStructHashable1));
}

uint32 GetTypeHash(const FTestStructHashable2& other)
{
	return FCrc::MemCrc32(&other, sizeof(FTestStructHashable2));
}
