// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"
#include "StructUtils/SharedStruct.h"
#include "StructUtilsTestTypes.h"

#define LOCTEXT_NAMESPACE "StructUtilsTests"

UE_DISABLE_OPTIMIZATION_SHIP

namespace FInstancedStructTest
{
	struct FTest_StructViewCreate : FAITestBase
	{
		virtual bool InstantTest() override
		{
			{
				FTestStructSimple TestStruct;

				TStructView<FTestStructSimpleBase> StructView;
				StructView = TStructView<FTestStructSimpleBase>(TestStruct);

				AITEST_EQUAL(
					"TStructView initialized from a value who's type is a child of the templated type should return the child type, not the templated type when calling GetScriptStruct()",
					StructView.GetScriptStruct(),
					FTestStructSimple::StaticStruct());
			}

			{
				FTestStructSimple TestStruct;

				TConstStructView<FTestStructSimpleBase> ConstStructView;
				ConstStructView = TConstStructView<FTestStructSimpleBase>(TestStruct);

				AITEST_EQUAL(
					"TConstStructView initialized from a value who's type is a child of the templated type should return the child type, not the templated type when calling GetScriptStruct()",
					ConstStructView.GetScriptStruct(),
					FTestStructSimple::StaticStruct());
			}

			return true;
		}
	};
	IMPLEMENT_AI_INSTANT_TEST(FTest_StructViewCreate, "System.StructUtils.StructView.Create");
}

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
