// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SubclassOf.h"
#include "StructUtils/PropertyBag.h"
#include "StructUtilsTestTypes.generated.h"

USTRUCT()
struct FTestStructSimpleBase
{
	GENERATED_BODY()
};

USTRUCT()
struct FTestStructSimple : public FTestStructSimpleBase
{
	GENERATED_BODY()
	
	FTestStructSimple() = default;
	FTestStructSimple(const float InFloat) : Float(InFloat) {}

	bool operator==(const FTestStructSimple& Other) const
	{
		return Float == Other.Float;
	}
	
	UPROPERTY()
	float Float = 0.0f;
};

USTRUCT()
struct FTestStructSimpleNonZeroDefault : public FTestStructSimpleBase
{
	GENERATED_BODY()

	FTestStructSimpleNonZeroDefault() = default;

	bool operator==(const FTestStructSimpleNonZeroDefault& Other) const
	{
		return Float == Other.Float && Bool == Other.Bool;
	}

	bool operator!=(const FTestStructSimpleNonZeroDefault& Other) const
	{
		return !operator==(Other);
	}

	UPROPERTY()
	float Float = 100.0f;
	bool Bool = true;

};

USTRUCT()
struct FTestStructComplex
{
	GENERATED_BODY()
	
	FTestStructComplex() = default;
	FTestStructComplex(const FString& InString) : String(InString) {}
	
	UPROPERTY()
	FString String; 

	UPROPERTY()
	TArray<FString> StringArray;
};

USTRUCT()
struct FTestStructSimple1 : public FTestStructSimpleBase
{
	GENERATED_BODY()
	
	FTestStructSimple1() = default;
	
	UPROPERTY()
	float Float = 0.0f;
};

USTRUCT()
struct FTestStructSimple2 : public FTestStructSimpleBase
{
	GENERATED_BODY()
	
	FTestStructSimple2() = default;
	
	UPROPERTY()
	float Float = 0.0f;
};

USTRUCT()
struct FTestStructSimple3 : public FTestStructSimpleBase
{
	GENERATED_BODY()
	
	FTestStructSimple3() = default;
	
	UPROPERTY()
	float Float = 0.0f;
};

USTRUCT()
struct FTestStructSimple4 : public FTestStructSimpleBase
{
	GENERATED_BODY()
	
	FTestStructSimple4() = default;
	
	UPROPERTY()
	float Float = 0.0f;
};

USTRUCT()
struct FTestStructSimple5 : public FTestStructSimpleBase
{
	GENERATED_BODY()
	
	FTestStructSimple5() = default;
	
	UPROPERTY()
	float Float = 0.0f;
};

USTRUCT()
struct FTestStructSimple6 : public FTestStructSimpleBase
{
	GENERATED_BODY()
	
	FTestStructSimple6() = default;
	
	UPROPERTY()
	float Float = 0.0f;
};

USTRUCT()
struct FTestStructSimple7 : public FTestStructSimpleBase
{
	GENERATED_BODY()
	
	FTestStructSimple7() = default;
	
	UPROPERTY()
	float Float = 0.0f;
};

USTRUCT()
struct FTestStructHashable1 : public FTestStructSimpleBase
{
	GENERATED_BODY()
	
	FTestStructHashable1() = default;
	
	UPROPERTY()
	float Float = 0.0f;
};

uint32 GetTypeHash(const FTestStructHashable1& other);

USTRUCT()
struct FTestStructHashable2 : public FTestStructSimpleBase
{
	GENERATED_BODY()

	FTestStructHashable2() = default;

	UPROPERTY()
	int Int = 0;
};

uint32 GetTypeHash(const FTestStructHashable2& other);

USTRUCT()
struct FTestStructNonTrivialDestructor
{
	GENERATED_BODY()

	FTestStructNonTrivialDestructor()
	{
		NumConstructed++;
	}

	~FTestStructNonTrivialDestructor()
	{
		NumConstructed--;
	}

	static inline int32 NumConstructed = 0;
};

UCLASS()
class UBagTestObject1 : public UObject
{
	GENERATED_BODY()
};

UCLASS()
class UBagTestObject2 : public UObject
{
	GENERATED_BODY()
};

UCLASS()
class UBagTestObject1Derived : public UBagTestObject1
{
	GENERATED_BODY()
};

UCLASS()
class UBagTestObject1Derived2 : public UBagTestObject1
{
	GENERATED_BODY()
};

UENUM()
enum class EPropertyBagTest1 : uint8
{
	Foo,
	Bar,
	Baz,
};

UENUM()
enum class EPropertyBagTest2 : uint8
{
	Bingo,
	Bongo,
};

UCLASS()
class UTestObjectWithPropertyBag : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FInstancedPropertyBag Bag;
};

UCLASS()
class UTestObjectWithInstanceStruct : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FInstancedStruct Value;
};

USTRUCT()
struct FTestStructWithSubClassOf
{
	GENERATED_BODY()

	UPROPERTY()
	TSubclassOf<UBagTestObject1> ClassProperty;

	UPROPERTY()
	TSoftClassPtr<UBagTestObject1> SoftClassProperty;
};